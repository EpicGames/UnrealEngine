// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldConditionBase)

FWorldConditionBase::~FWorldConditionBase()
{
	// Empty
}

FText FWorldConditionBase::GetDescription() const
{
#if WITH_EDITORONLY_DATA
	return StaticStruct()->GetDisplayNameText();
#else
	return FText::FromString(StaticStruct()->GetAuthoredName());
#endif
}

bool FWorldConditionBase::Initialize(const UWorldConditionSchema& Schema)
{
	return true;
}

bool FWorldConditionBase::Activate(const FWorldConditionContext& Context) const
{
	return true;
}

FWorldConditionResult FWorldConditionBase::IsTrue(const FWorldConditionContext& Context) const
{
	return FWorldConditionResult(EWorldConditionResultValue::IsTrue, bCanCacheResult);
}

void FWorldConditionBase::Deactivate(const FWorldConditionContext& Context) const
{
	// Empty
}
