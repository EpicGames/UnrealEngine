// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionContext.h"
#include "VisualLogger/VisualLogger.h"

bool FWorldConditionContext::Activate() const
{
	if (!QueryState.IsInitialized())
	{
		return false;
	}

	if (QueryState.AreConditionsActivated())
	{
		UE_VLOG_UELOG(QueryState.GetOwner(), LogWorldCondition, Warning, TEXT("Conditions already actived. Validate the call site to avoid redundant activation."));
		return true;
	}

	const FWorldConditionQuerySharedDefinition* SharedDefinition = QueryState.GetSharedDefinition();
	if (SharedDefinition == nullptr)
	{
		// Initialized but no definition means empty query. Activating empty should succeed.
		return true;
	}

	const FInstancedStructContainer& Conditions = SharedDefinition->GetConditions();
	
	for (int32 Index = 0; Index < Conditions.Num(); Index++)
	{
		const FWorldConditionBase& Condition = Conditions[Index].Get<const FWorldConditionBase>();
		FWorldConditionItem& Item = QueryState.GetItem(Index);
		Item.Operator = Condition.GetOperator();
		Item.NextExpressionDepth = Condition.GetNextExpressionDepth();
	}

	bool bSuccess = true;
	for (int32 Index = 0; Index < Conditions.Num(); Index++)
	{
		const FWorldConditionBase& Condition = Conditions[Index].Get<const FWorldConditionBase>();
		if (!Condition.Activate(*this))
		{
			UE_VLOG_UELOG(QueryState.GetOwner(), LogWorldCondition, Error, TEXT("Failed to activate condition at index %d"), Index);
			bSuccess = false;
		}
	}

	if (LIKELY(bSuccess))
	{
		QueryState.SetConditionsActivated(true);
	}
	else
	{
		Deactivate();
	}
	
	return bSuccess;
}

bool FWorldConditionContext::IsTrue() const
{
	if (!QueryState.IsInitialized())
	{
		return false;
	}

	if (QueryState.GetCachedResult() != EWorldConditionResultValue::Invalid)
	{
		return QueryState.GetCachedResult() == EWorldConditionResultValue::IsTrue;
	}

	const FWorldConditionQuerySharedDefinition* SharedDefinition = QueryState.GetSharedDefinition();
	if (SharedDefinition == nullptr)
	{
		// Empty query is true.
		return true;
	}

	if (!ensureMsgf(QueryState.AreConditionsActivated(), TEXT("Can not evaluate non activated conditions. Activate must be called first.")))
	{
		return false;
	}

	static_assert(UE::WorldCondition::MaxExpressionDepth == 4);
	EWorldConditionResultValue Results[UE::WorldCondition::MaxExpressionDepth + 1] = { EWorldConditionResultValue::Invalid, EWorldConditionResultValue::Invalid, EWorldConditionResultValue::Invalid, EWorldConditionResultValue::Invalid, EWorldConditionResultValue::Invalid };
	EWorldConditionOperator Operators[UE::WorldCondition::MaxExpressionDepth + 1] = { EWorldConditionOperator::Copy, EWorldConditionOperator::Copy, EWorldConditionOperator::Copy, EWorldConditionOperator::Copy, EWorldConditionOperator::Copy };
	int32 Depth = 0;

	bool bAllConditionsCanBeCached = true;

	for (int32 Index = 0; Index < QueryState.GetNumConditions(); Index++)
	{
		FWorldConditionItem& Item = QueryState.GetItem(Index);
		const int32 NextExpressionDepth = Item.NextExpressionDepth;

		Operators[Depth] = Item.Operator;

		Depth = FMath::Max(Depth, NextExpressionDepth);
		
		EWorldConditionResultValue CurrResult = Item.CachedResult;
		if (CurrResult == EWorldConditionResultValue::Invalid)
		{
			const FInstancedStructContainer& Conditions = SharedDefinition->GetConditions();
			check(Conditions.Num() == QueryState.GetNumConditions());
			const FWorldConditionBase& Condition = Conditions[Index].Get<const FWorldConditionBase>();
			FWorldConditionResult ConditionResult = Condition.IsTrue(*this);
			CurrResult = UE::WorldCondition::Invert(ConditionResult.Value, Condition.ShouldInvertResult()); 

			// Cache result if possible; clear cache otherwise
			Item.CachedResult = ConditionResult.bCanBeCached ? CurrResult : EWorldConditionResultValue::Invalid;
			
			bAllConditionsCanBeCached &= ConditionResult.bCanBeCached;
		}

		Depth++;
		Results[Depth] = CurrResult;

		while (Depth > NextExpressionDepth)
		{
			Depth--;
			Results[Depth] = UE::WorldCondition::MergeResults(Operators[Depth], Results[Depth], Results[Depth + 1]);
			Operators[Depth] = EWorldConditionOperator::Copy;
		}
	}

	const EWorldConditionResultValue FinalResult = Results[0];
	
	QueryState.SetCachedResult(bAllConditionsCanBeCached ? FinalResult : EWorldConditionResultValue::Invalid);
	
	return FinalResult == EWorldConditionResultValue::IsTrue;

}

void FWorldConditionContext::Deactivate() const
{
	if (!QueryState.IsInitialized())
	{
		return;
	}

	if (const FWorldConditionQuerySharedDefinition* SharedDefinition = QueryState.GetSharedDefinition())
	{
		const FInstancedStructContainer& Conditions = SharedDefinition->GetConditions();
		for (int32 Index = 0; Index < Conditions.Num(); Index++)
		{
			const FWorldConditionBase& ConditionDef = Conditions[Index].Get<const FWorldConditionBase>();
			ConditionDef.Deactivate(*this);
		}
	}

	QueryState.SetConditionsActivated(false);
	QueryState.Free();
}
