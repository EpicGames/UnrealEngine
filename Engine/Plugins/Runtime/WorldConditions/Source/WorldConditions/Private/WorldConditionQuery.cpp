// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionQuery.h"
#include "Containers/StringFwd.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "PropertyBindingPath.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Serialization/CustomVersion.h"
#include "UObject/GarbageCollection.h"
#include "UObject/UObjectThreadContext.h"
#include "WorldConditionContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldConditionQuery)

#define LOCTEXT_NAMESPACE "WorldCondition"

struct FWorldConditionCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Changed shared definition to a struct.
		StructSharedDefinition = 1,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
	// The GUID for this custom version number
	const static FGuid GUID;
private:
	FWorldConditionCustomVersion() {}
};

const FGuid FWorldConditionCustomVersion::GUID(0x2C28AC22, 0x15CF46FE, 0xBD19F011, 0x652A3C05);

// Register the custom version with core
FCustomVersionRegistration GWorldConditionCustomVersion(FWorldConditionCustomVersion::GUID, FWorldConditionCustomVersion::LatestVersion, TEXT("WorldConditionCustomVersion"));


//
// FWorldConditionResultInvalidationHandle
//

void FWorldConditionResultInvalidationHandle::InvalidateResult() const
{
	const TSharedPtr<uint8> StateMemory = WeakStateMemory.Pin();
	if (StateMemory.IsValid())
	{
		EWorldConditionResultValue* CachedResult = reinterpret_cast<EWorldConditionResultValue*>(StateMemory.Get() + FWorldConditionQueryState::CachedResultOffset);
		FWorldConditionItem* Item = reinterpret_cast<FWorldConditionItem*>(StateMemory.Get() + ItemOffset);
		*CachedResult = EWorldConditionResultValue::Invalid;
		Item->CachedResult = EWorldConditionResultValue::Invalid;
	}
	else
	{
		UE_LOGF(LogWorldCondition, Warning, "World Condition: Trying to access freed state memory while calling InvalidateResult(). "
			"Make sure you are calling Deactivate() on your world condition query, and that you unregister any delegates on world conditions.");
	}
}


//
// FWorldConditionQueryState
//

FWorldConditionQueryState::~FWorldConditionQueryState()
{
	// Usually the game code should call Deactivate() as part of the uninitialization of the owner of the query state.
	// Try to clean up the best we can in case things are still running.
	const FWorldConditionQuerySharedDefinition* Definition = GetSharedDefinition();
	if (IsInitialized() && Definition != nullptr)
	{
		if (AreConditionsActivated())
		{
			// During GC the Owner may already have been destroyed in the same purge pass,
			// leaving a dangling pointer. IsValid() would dereference it and crash, so we
			// can only inspect the Owner when we are not running inside garbage collection.
			const bool bCanInspectOwner = !IsGarbageCollecting();
			if (bCanInspectOwner && IsValid(Owner))
			{
				// We can call Deactivate(), but we don't know the context data, so some uninitialization can potentially be incomplete.
				UE_LOGF(LogWorldCondition, Warning, "World Condition: State %p owned by %ls is still active on destructor, "
					"calling Deactivate() without context data, might leak memory or resources.",
					this,
					*GetFullNameSafe(Owner));

				const FWorldConditionContextData ContextData(*Definition->GetSchemaClass().GetDefaultObject());
				const FWorldConditionContext Context(*this, ContextData);
				Context.Deactivate();
			}
			else
			{
				// Either the owner is not valid or we are inside GC and cannot safely inspect it.
				// Fall back to Free() which only releases our own heap memory and is GC-safe.
				UE_LOGF(LogWorldCondition, Warning, "World Condition: State %p is still active on destructor (%ls), "
					"calling Free() without Deactivate(), might leak memory or resources.",
					this,
					bCanInspectOwner ? TEXT("invalid Owner") : TEXT("inside garbage collection"));

				Free();
			}
		}
		else
		{
			// Activate() was never (e.g. Lazy activated conditions) called so only need to call Free()
			Free();
		}
	}

	if (Memory.IsValid())
	{
		// Something went very wrong, this should never happen.
		UE_LOGF(LogWorldCondition, Error, "World Condition: State %p has still allocated memory in destructor, might leak memory.", Memory.Get());
		Memory = nullptr;
	}
}
	
void FWorldConditionQueryState::Initialize(const UObject& InOwner, const FWorldConditionQueryDefinition& QueryDefinition)
{
	InitializeInternal(&InOwner, QueryDefinition.GetSharedDefinitionStruct());
}

void FWorldConditionQueryState::InitializeInternal(const UObject* InOwner, const FSharedStruct InSharedDefinition)
{
	if (IsInitialized())
	{
		Free();
	}

	Owner = InOwner;

	FWorldConditionQuerySharedDefinition* Definition = InSharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	if (Definition == nullptr
		|| Definition->GetStateSize() == 0)
	{
		// Empty condition
		SharedDefinition.Reset();
		NumConditions = 0;
		bIsInitialized = true;
		return;
	}

	if (!Definition->IsLinked())
	{
		UE_LOGF(LogWorldCondition, Error, "World Condition: Trying to initialize query state with invalid definition for %ls.", *GetNameSafe(Owner));
		return;
	}
	
	SharedDefinition = InSharedDefinition;

	const FInstancedStructContainer& Conditions = Definition->GetConditions();

	// Cache num conditions so that we can access the items and cached data without touching the definition.
	NumConditions = IntCastChecked<uint8>(Conditions.Num());

	uint8* AllocatedMemory = static_cast<uint8*>(FMemory::Malloc(Definition->GetStateSize(), Definition->GetStateMinAlignment()));
	Memory = TSharedPtr<uint8>(AllocatedMemory,
		/* Deleter */[](uint8* Obj)
		{
			FMemory::Free(Obj);
		});

	// Init cached result
	EWorldConditionResultValue& CachedResult = *reinterpret_cast<EWorldConditionResultValue*>(AllocatedMemory + CachedResultOffset);
	CachedResult = EWorldConditionResultValue::Invalid;
	
	// Initialize items
	for (int32 Index = 0; Index < static_cast<int32>(NumConditions); Index++)
	{
		new (AllocatedMemory + ItemsOffset + sizeof(FWorldConditionItem) * Index) FWorldConditionItem();
	}

	// Initialize state
	for (int32 Index = 0; Index < static_cast<int32>(NumConditions); Index++)
	{
		const FWorldConditionBase& Condition = Conditions[Index].Get<const FWorldConditionBase>();
		if (Condition.GetStateDataOffset() == 0)
		{
			continue;
		}
		uint8* StateMemory = AllocatedMemory + Condition.GetStateDataOffset();
		if (Condition.IsStateObject())
		{
			new (StateMemory) FWorldConditionStateObject();
			FWorldConditionStateObject& StateObject = *reinterpret_cast<FWorldConditionStateObject*>(StateMemory);
			const UClass* StateClass = Cast<UClass>(Condition.GetRuntimeStateType()->Get());
			StateObject.Object = NewObject<UObject>(const_cast<UObject*>(Owner.Get()), StateClass);  
		}
		else
		{
			const UScriptStruct* StateScriptStruct = Cast<UScriptStruct>(Condition.GetRuntimeStateType()->Get());
			StateScriptStruct->InitializeStruct(StateMemory);
		}
	}

	bIsInitialized = true;
}

void FWorldConditionQueryState::Free()
{
	if (!Memory.IsValid())
	{
		NumConditions = 0;
		SharedDefinition.Reset();
		bIsInitialized = false;
		return;
	}

	const FWorldConditionQuerySharedDefinition* Definition = SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	check(Definition != nullptr);

	// Items don't need destructing.

	// Destroy state
	const FInstancedStructContainer& Conditions = Definition->GetConditions();
	for (int32 Index = 0; Index < static_cast<int32>(NumConditions); Index++)
	{
		const FWorldConditionBase& Condition = Conditions[Index].Get<const FWorldConditionBase>();
		if (Condition.GetStateDataOffset() == 0)
		{
			continue;
		}
		uint8* StateMemory = Memory.Get() + Condition.GetStateDataOffset();
		if (Condition.IsStateObject())
		{
			FWorldConditionStateObject& StateObject = *reinterpret_cast<FWorldConditionStateObject*>(Memory.Get() + Condition.GetStateDataOffset());
			StateObject.~FWorldConditionStateObject();
		}
		else
		{
			const UScriptStruct* StateScriptStruct = Cast<UScriptStruct>(Condition.GetRuntimeStateType()->Get());
			StateScriptStruct->DestroyStruct(StateMemory);
		}
	}

	Memory = nullptr;
	NumConditions = 0;
	SharedDefinition.Reset();
	bIsInitialized = false;
}

void FWorldConditionQueryState::AddStructReferencedObjects(class FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Owner);

	FWorldConditionQuerySharedDefinition* Definition = SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	if (Definition != nullptr)
	{
		Collector.AddPropertyReferencesWithStructARO(FWorldConditionQuerySharedDefinition::StaticStruct(), Definition);
	}
	
	if (!Memory.IsValid() || Definition == nullptr)
	{
		return;
	}

	const FInstancedStructContainer& Conditions = Definition->GetConditions();
	
	check(NumConditions == Conditions.Num());

	for (int32 Index = 0, Num = Conditions.Num(); Index < Num; Index++)
	{
		const FWorldConditionBase& Condition = Conditions[Index].Get<const FWorldConditionBase>();
		if (Condition.GetStateDataOffset() == 0)
		{
			continue;
		}
		
		uint8* StateMemory = Memory.Get() + Condition.GetStateDataOffset();
		if (Condition.IsStateObject())
		{
			FWorldConditionStateObject& StateObject = *reinterpret_cast<FWorldConditionStateObject*>(StateMemory);
			Collector.AddReferencedObject(StateObject.Object);
		}
		else
		{
			if (auto* StateScriptStructPtr = Condition.GetRuntimeStateType(); StateScriptStructPtr && *StateScriptStructPtr)
			{
				if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(*StateScriptStructPtr))
				{
					Collector.AddReferencedObject(*StateScriptStructPtr, Owner);
					Collector.AddPropertyReferencesWithStructARO(ScriptStruct, StateMemory, Owner);
				}
			}
		}
	}
}

FWorldConditionResultInvalidationHandle FWorldConditionQueryState::GetInvalidationHandle(const FWorldConditionBase& Condition) const
{
	check(bIsInitialized);
	check(Memory.IsValid() && Condition.GetConditionIndex() < NumConditions);
	const int32 ItemOffset = ItemsOffset + Condition.GetConditionIndex() * sizeof(FWorldConditionItem);
	return FWorldConditionResultInvalidationHandle(Memory, ItemOffset);
}


//
// UWorldConditionQuerySharedDefinition
//

void FWorldConditionQuerySharedDefinition::Set(const TSubclassOf<UWorldConditionSchema> InSchema, const TArrayView<FConstStructView> InConditions)
{
	SchemaClass = InSchema;
	Conditions = InConditions;
	StateMinAlignment = 8;
	StateSize = 0;
}

void FWorldConditionQuerySharedDefinition::Set(const TSubclassOf<UWorldConditionSchema> InSchema, const TArrayView<FStructView> InConditions)
{
	SchemaClass = InSchema;
	Conditions = InConditions;
	StateMinAlignment = 8;
	StateSize = 0;
}

void FWorldConditionQuerySharedDefinition::PostSerialize(const FArchive& Ar)
{
	// We need to link when loading persistent data and when loading during duplication (DuplicateObject or BP re-instancing)
	if (Ar.IsLoading())
	{
		const FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
		const UObject* SerializedObject = LoadContext ? LoadContext->SerializedObject : nullptr;

		if (!Link(SerializedObject))
		{
			UE_SUPPRESS(LogWorldCondition, Error,
			{
				TStringBuilder<256> PropertyPath;
				if (const FArchiveSerializedPropertyChain* PropertyChain = Ar.GetSerializedPropertyChain())
				{
					const int32 MaxProperties = PropertyChain->GetNumProperties();
					for (int32 PropertyNum = 0; PropertyNum < MaxProperties; PropertyNum++)
					{
						if (const FProperty* CurrProperty = PropertyChain->GetPropertyFromRoot(PropertyNum))
						{
							if (PropertyPath.Len() > 0)
							{
								PropertyPath << TEXT('.');
							}
							PropertyPath << CurrProperty->GetFName();
						}
					}
				}

				UE_LOGF(LogWorldCondition, Error, "World Condition: Failed to link query for property '%ls' in '%ls'"
					, *PropertyPath
					, *GetPathNameSafe(SerializedObject));
			});
		}
	}
}

bool FWorldConditionQuerySharedDefinition::Identical(const FWorldConditionQuerySharedDefinition* Other, uint32 PortFlags) const
{
	if (!Other)
	{
		return false;
	}

	if (SchemaClass != Other->SchemaClass
		|| StateMinAlignment != Other->StateMinAlignment
		|| StateSize != Other->StateSize
		|| bIsLinked != Other->bIsLinked)
	{
		return false;
	}

	return Conditions.Identical(&Other->Conditions, PortFlags);
}


bool FWorldConditionQuerySharedDefinition::Link(const UObject* Outer)
{
	bool bResult = true;

	StateMinAlignment = 0;
	StateSize = 0;
	bIsLinked = false;

	const UWorldConditionSchema* Schema = SchemaClass.GetDefaultObject();
	if (!Schema)
	{
		UE_LOGF(LogWorldCondition, Error, "World Condition: invalid schema class %ls in a query definition for %ls."
			, *GetPathNameSafe(*SchemaClass)
			, *GetPathNameSafe(Outer));
		return false;
	}

	// Calculate layout
	int32 MinAlignment = 8;
	int32 Offset = 0;

	// Reserve space for cached result
	Offset += sizeof(EWorldConditionResultValue);
	
	// Reserve space for condition items.
	Offset = Align(Offset, alignof(FWorldConditionItem));
	check(Offset == FWorldConditionQueryState::ItemsOffset);
	Offset += sizeof(FWorldConditionItem) * Conditions.Num(); 

	// Reserve space for all runtime data.
	for (int32 Index = 0; Index < Conditions.Num(); Index++)
	{
		FWorldConditionBase* Condition = Conditions[Index].GetPtr<FWorldConditionBase>();
		if (!Condition)
		{
			UE_LOGF(LogWorldCondition, Error, "World Condition: Encountered empty condition in a query definition for %ls. This may be caused by the query using a struct from a plugin that is not loaded, or the condition type is removed.",
				*GetNameSafe(Outer));
			return false;
		}
		Condition->ConditionIndex = Index;

		if (auto* StateStruct = Condition->GetRuntimeStateType(); StateStruct && *StateStruct)
		{
			int32 StructMinAlignment = 0;
			int32 StructSize = 0;

			if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(*StateStruct))
			{
				StructMinAlignment = ScriptStruct->GetMinAlignment();
				StructSize = ScriptStruct->GetStructureSize();
				Condition->bIsStateObject = false;
			}
			else if (const UClass* Class = Cast<const UClass>(*StateStruct))
			{
				StructMinAlignment = FWorldConditionStateObject::StaticStruct()->GetMinAlignment();
				StructSize = FWorldConditionStateObject::StaticStruct()->GetStructureSize();
				Condition->bIsStateObject = true;
			}
			
			check(StructMinAlignment > 0 && StructSize > 0);

			Offset = Align(Offset, StructMinAlignment);
			Condition->StateDataOffset = IntCastChecked<uint16>(Offset);

			Offset += StructSize;
			MinAlignment = FMath::Max(MinAlignment, StructMinAlignment);
		}
		else
		{
			Condition->StateDataOffset = 0;
		}
	}

	StateMinAlignment = uint8(MinAlignment);
	StateSize = uint16(Offset);
	
	for (int32 Index = 0; Index < Conditions.Num(); Index++)
	{
		FWorldConditionBase& Condition = Conditions[Index].Get<FWorldConditionBase>();
		bResult &= Condition.Initialize(*Schema);
	}

	bIsLinked = bResult;

	return bResult;
}

//
// FWorldConditionQueryDefinition
//

bool FWorldConditionQueryDefinition::IsValid() const
{
	const FWorldConditionQuerySharedDefinition* Definition = SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	return Definition != nullptr && Definition->IsLinked();
}

void FWorldConditionQueryDefinition::SetSchemaClass(const TSubclassOf<UWorldConditionSchema> InSchema)
{
	SchemaClass = InSchema;
}

#if WITH_EDITORONLY_DATA
bool FWorldConditionQueryDefinition::Initialize(const UObject* Outer, const TSubclassOf<UWorldConditionSchema> InSchemaClass, const TConstArrayView<FWorldConditionEditable> InConditions)
{
	SchemaClass = InSchemaClass;
	EditableConditions = InConditions;
	return Initialize(Outer);
}
#endif

FText FWorldConditionQueryDefinition::GetDescription() const
{
	const FWorldConditionQuerySharedDefinition* Definition = SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	if (Definition == nullptr)
	{
		return LOCTEXT("Empty", "Empty");
	}

	const FInstancedStructContainer& Conditions = Definition->GetConditions();
	if (Conditions.Num() == 0)
	{
		return LOCTEXT("Empty", "Empty");
	}

	TStringBuilder<256> Builder;
	
	const FWorldConditionBase* PrevCondition = nullptr;
	for (int32 Index = 0; Index < Conditions.Num(); Index++)
	{
		const FWorldConditionBase& Condition = Conditions[Index].Get<const FWorldConditionBase>();
		const int32 CurrDepth = PrevCondition ? PrevCondition->GetNextExpressionDepth() : 0;
		const int32 NextDepth = Condition.GetNextExpressionDepth();
		const int32 DeltaDepth = NextDepth - CurrDepth;
		const int32 OpenParens = FMath::Max(0, DeltaDepth);
		const int32 CloseParens = FMath::Max(0, -DeltaDepth);

		// Operator
		FText Operator;
		if (Index == 0)
		{
			Operator = LOCTEXT("IfOperator", "IF");
		}
		else if (Condition.GetOperator() == EWorldConditionOperator::And)
		{
			Operator = LOCTEXT("AndOperator", "AND");
		}
		else if (Condition.GetOperator() == EWorldConditionOperator::Or)
		{
			Operator = LOCTEXT("OrOperator", "OR");
		}
		else
		{
			ensureMsgf(false, TEXT("Unhandled operator %s\n"), *UEnum::GetValueAsString(Condition.GetOperator()));
			break;
		}
		Builder.Append(Operator.ToString());
		Builder.AppendChar(' ');

		// Open parens
		for (int32 Paren = 0; Paren < OpenParens; Paren++)
		{
			Builder.AppendChar('(');
		}

		// Item desc
		FText ConditionDesc = Condition.GetDescription();
		Builder.AppendChar('[');
		Builder.Append(ConditionDesc.ToString());
		Builder.AppendChar(']');

		// Close parens
		for (int32 Paren = 0; Paren < CloseParens; Paren++)
		{
			Builder.AppendChar(')');
		}

		if ((Index + 1) < Conditions.Num())
		{
			Builder.AppendChar(' ');
		}

		PrevCondition = &Condition;
	}

	return FText::FromString(Builder.ToString());
}

bool FWorldConditionQueryDefinition::Initialize(const UObject* Outer, const EInitializationMode InitializationMode)
{
	if (InitializationMode == EInitializationMode::UseExistingSharedDefinitionOnEmptyConditions
		&& EditableConditions.IsEmpty())
	{
		return true;
	}

	FConstSharedStruct OldSharedDefinition = SharedDefinition;
	SharedDefinition.Reset();

	if (!::IsValid(SchemaClass))
	{
		UE_LOGF(LogWorldCondition, Warning, "World Condition: Failed to initialize query for %ls due to missing schema.", *GetFullNameSafe(Outer));
		return false;
	}

	const UWorldConditionSchema* Schema = SchemaClass.GetDefaultObject();
	
	// Append only valid condition.
	TArray<FStructView> ValidConditions;
	ValidConditions.Reserve(EditableConditions.Num());
	for (FWorldConditionEditable& EditableCondition : EditableConditions)
	{
		if (EditableCondition.Condition.IsValid())
		{
			if (Schema->IsStructAllowed(EditableCondition.Condition.GetScriptStruct()))
			{
				FWorldConditionBase& Condition = EditableCondition.Condition.GetMutable<FWorldConditionBase>();
				// Store expression depth temporarily into NextExpressionDepth, it will be update below.
				Condition.NextExpressionDepth = EditableCondition.ExpressionDepth;
				Condition.Operator = EditableCondition.Operator;
				Condition.bInvert = EditableCondition.bInvert;

				ValidConditions.Add(EditableCondition.Condition);
			}
			else
			{
				UE_LOGF(LogWorldCondition, Warning, "World Condition: Query for %ls contains condition of type %ls that is not allowed by schema %ls.",
					*GetFullNameSafe(Outer), *GetNameSafe(EditableCondition.Condition.GetScriptStruct()), *GetNameSafe(Schema));
			}
		}
	}
	
	if (ValidConditions.IsEmpty())
	{
		// Empty query, do not create definition.
		return true;
	}

	
	// Prepare the conditions for evaluation.
	if (ValidConditions.Num() > 0)
	{
		FWorldConditionBase& Condition = ValidConditions[0].Get<FWorldConditionBase>();
		Condition.Operator = EWorldConditionOperator::Copy;
	}

	for (int32 Index = 0; Index < ValidConditions.Num(); Index++)
	{
		uint8 NextExpressionDepth = 0;
		if ((Index + 1) < ValidConditions.Num())
		{
			const FWorldConditionBase& NextCondition = ValidConditions[Index + 1].Get<FWorldConditionBase>();
			NextExpressionDepth = NextCondition.NextExpressionDepth;
		}
		
		FWorldConditionBase& Condition = ValidConditions[Index].Get<FWorldConditionBase>();
		Condition.NextExpressionDepth = NextExpressionDepth;

		Condition.ConditionIndex = Index;
	}

	// Create a new shared definition to allow the allocated states to deactivate properly even if we update the definition.
	SharedDefinition = FSharedStruct::Make<FWorldConditionQuerySharedDefinition>();
	FWorldConditionQuerySharedDefinition& Definition = SharedDefinition.Get<FWorldConditionQuerySharedDefinition>();
	Definition.Set(SchemaClass, ValidConditions);
	Definition.Link(Outer);

	return true;
}

bool FWorldConditionQueryDefinition::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FWorldConditionCustomVersion::GUID);

#if WITH_EDITORONLY_DATA
	// Editable conditions should not exist by default when loading WorldConditions
	// in cooked builds other than for hotfix purposes.
	// In order to fulfill the primary contract of Cook-Save (package data should not be modified during the save)
	// we back up and clear the conditions before writing the cooked data and restore them after
	TArray<FWorldConditionEditable> ConditionsToRestore;
	if (Ar.IsSaving()
		&& Ar.IsCooking())
	{
		ConditionsToRestore = MoveTemp(EditableConditions);
	}
	ON_SCOPE_EXIT
	{
		if (ConditionsToRestore.Num())
		{
			EditableConditions = MoveTemp(ConditionsToRestore);
		}
	};

	// Lazy-build SharedDefinition BEFORE the property serialization runs.
	// Save runs Serialize twice - once for the name harvest pass and once for the actual
	// write - and both must reference the same set of FNames. If we built SharedDefinition
	// AFTER serialize, the harvest pass would see SharedDefinition.IsValid() == false and
	// skip the inner SerializeTaggedProperties; the write pass would then see it valid (we
	// just built it) and reference names like "Conditions" and "Index" that harvest never
	// recorded, tripping LinkerSave's "Name X is not mapped" critical-error ensure and
	// flipping save_assets to False.
	if (Ar.IsSaving()
		&& Ar.IsPersistent()
		&& !SharedDefinition.IsValid()
		&& EditableConditions.Num() > 0)
	{
		const FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
		const UObject* Outer = LoadContext ? LoadContext->SerializedObject : nullptr;
		Initialize(Outer);
	}
#endif // WITH_EDITORONLY_DATA

	// Use default serialization for most properties for
	// archive configurations supported by UStruct::SerializeVersionedTaggedProperties
	if (Ar.IsLoading() || Ar.IsSaving() || Ar.IsCountingMemory() || Ar.IsObjectReferenceCollector())
	{
		if (Ar.WantBinaryPropertySerialization())
		{
			StaticStruct()->SerializeBin(Ar, this);
		}
		else
		{
			StaticStruct()->SerializeTaggedProperties(Ar, (uint8*)this, StaticStruct(), nullptr);
		}
	}

	if (Ar.CustomVer(FWorldConditionCustomVersion::GUID) >= FWorldConditionCustomVersion::StructSharedDefinition)
	{
		// Serialize shared definition
		bool bHasSharedDefinition = SharedDefinition.IsValid();
		Ar << bHasSharedDefinition;

		if (Ar.IsLoading())
		{
			if (bHasSharedDefinition)
			{
				SharedDefinition = FSharedStruct::Make<FWorldConditionQuerySharedDefinition>();
			}
			else
			{
				SharedDefinition.Reset();
			}
		}

		FWorldConditionQuerySharedDefinition* RawDefinition = SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
		if (RawDefinition
			&& (Ar.IsLoading() || Ar.IsSaving() || Ar.IsCountingMemory() || Ar.IsObjectReferenceCollector()))
		{
			UScriptStruct* Struct = TBaseStructure<FWorldConditionQuerySharedDefinition>::Get();

			if (Ar.WantBinaryPropertySerialization())
			{
				Struct->SerializeBin(Ar, RawDefinition);
			}
			else
			{
				Struct->SerializeTaggedProperties(Ar, (uint8*)RawDefinition, Struct, nullptr);
			}

			// SerializeTaggedProperties does not call PostSerialize() on the struct it's called (calls in items), call it manually.
			RawDefinition->PostSerialize(Ar);
		}
	}

#if WITH_EDITORONLY_DATA
	// Loading-side lazy build. The save-side equivalent moved to before the property
	// serialization above to keep the harvest and write passes deterministic.
	if (Ar.IsLoading()
		&& !SharedDefinition.IsValid()
		&& EditableConditions.Num() > 0)
	{
		const FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
		const UObject* Outer = LoadContext ? LoadContext->SerializedObject : nullptr;
		Initialize(Outer);
	}
#endif // WITH_EDITORONLY_DATA

	return true;
}

bool FWorldConditionQueryDefinition::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive)
{
	if (const TCHAR* Result = TBaseStructure<FWorldConditionQueryDefinition>::Get()->ImportText(Buffer, this, Parent, PortFlags, ErrorText, TEXT("FWorldConditionQueryDefinition"), /*bAllowNativeOverride*/false))
	{
#if WITH_EDITORONLY_DATA
		Initialize(Parent);
#else
		// In cooked builds, this code path is allowed to reset the SharedDefinition
		// if EditableConditions is empty in the data imported as text from a hotfix.
		// e.g., this is required for the case where the definition was cooked with valid conditions,
		// then they get all removed by the hotfix.
		Initialize(Parent, EInitializationMode::ResetSharedDefinitionOnEmptyConditions);

		// Editable conditions are no longer necessary after the initialization (i.e., hotfix applied)
		EditableConditions.Reset();
#endif

		Buffer = Result;
		return true;
	}
	
	return false;
}

bool FWorldConditionQueryDefinition::ExportTextItem(FString& ValueStr, FWorldConditionQueryDefinition const& DefaultValue, class UObject* Parent, int32 PortFlags, class UObject* ExportRootScope) const
{
#if WITH_EDITOR
	if (PortFlags & PPF_PropertyWindow)
	{
		const FText Desc = GetDescription();
		ValueStr = Desc.ToString();
		return true;
	}
#endif
	
	return false;
}

void FWorldConditionQueryDefinition::AddStructReferencedObjects(FReferenceCollector& Collector) const
{
	if (FWorldConditionQuerySharedDefinition* Definition = SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>())
	{
		Collector.AddPropertyReferencesWithStructARO(FWorldConditionQuerySharedDefinition::StaticStruct(), Definition);
	}
}

bool FWorldConditionQueryDefinition::Identical(const FWorldConditionQueryDefinition* Other, uint32 PortFlags) const
{
	if (!Other)
	{
		return false;
	}

	if (SchemaClass != Other->SchemaClass
		|| SharedDefinition.IsValid() != Other->SharedDefinition.IsValid())
	{
		return false;
	}

	const FWorldConditionQuerySharedDefinition* Definition = SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	const FWorldConditionQuerySharedDefinition* OtherDefinition = Other->SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	if (Definition != nullptr
		&& OtherDefinition != nullptr
		&& !Definition->Identical(OtherDefinition, PortFlags))
	{
		return false;
	}
	
#if WITH_EDITORONLY_DATA
	if (EditableConditions != Other->EditableConditions)
	{
		return false;
	}
#endif

	return true;
}

bool FWorldConditionQueryDefinition::TryRetargetingPathToConditions(FPropertyBindingPath& InOutPath)
{
	bool bConditionPathUpdated = false;

	// Bindings set in Editor points to "EditableConditions", we want to return a path to the runtime version
	// e.g., we want to retarget:
	//	"Foo.EditableConditions[0].Condition(InstanceStruct="SomeStruct").Bar"
	// to
	//	"Foo.SharedDefinition(InstanceStruct="WorldConditionQuerySharedDefinition").Condition[0](InstanceStruct="SomeStruct").Bar"
	TArrayView<FPropertyBindingPathSegment> AllSegments = InOutPath.GetMutableSegments();

	// not using GET_MEMBER_NAME_CHECKED() since EditableConditions is only available in Editor builds
	const FName EditableConditionsName("EditableConditions");
	for (int32 SegmentIndex = 0; SegmentIndex < AllSegments.Num(); ++SegmentIndex)
	{
		FPropertyBindingPathSegment& Segment = AllSegments[SegmentIndex];
		if (Segment.GetName() == EditableConditionsName)
		{
			const int32 NextSegmentIndex = SegmentIndex + 1;
			if (AllSegments.IsValidIndex(NextSegmentIndex))
			{
				const int32 EditableConditionIndex = Segment.GetArrayIndex();
				const UStruct* InternalConditionStruct = AllSegments[NextSegmentIndex].GetInstanceStruct();

				AllSegments[SegmentIndex] = FPropertyBindingPathSegment(GET_MEMBER_NAME_CHECKED(FWorldConditionQueryDefinition, SharedDefinition)
					, INDEX_NONE
					, FWorldConditionQuerySharedDefinition::StaticStruct()
					, EPropertyBindingPropertyAccessType::SharedStruct);

				AllSegments[NextSegmentIndex] = FPropertyBindingPathSegment(GET_MEMBER_NAME_CHECKED(FWorldConditionQuerySharedDefinition, Conditions)
					, EditableConditionIndex
					, InternalConditionStruct
					, EPropertyBindingPropertyAccessType::StructInstanceContainer);

				bConditionPathUpdated = true;
			}
			break;
		}
	}

	return bConditionPathUpdated;
}

//
// FWorldConditionQuery
//

#if WITH_EDITORONLY_DATA
bool FWorldConditionQuery::DebugInitialize(const UObject* Outer, const TSubclassOf<UWorldConditionSchema> InSchemaClass, const TConstArrayView<FWorldConditionEditable> InConditions)
{
	if (IsActive())
	{
		return false;
	}

	return QueryDefinition.Initialize(Outer, InSchemaClass, InConditions);
}
#endif // WITH_EDITORONLY_DATA

bool FWorldConditionQuery::Activate(const UObject& InOwner, const FWorldConditionContextData& ContextData) const
{
	QueryState.Initialize(InOwner, QueryDefinition);
	if (!QueryState.IsInitialized())
	{
		return false;
	}

	const FWorldConditionContext Context(QueryState, ContextData);
	return Context.Activate();
}

bool FWorldConditionQuery::IsTrue(const FWorldConditionContextData& ContextData) const
{
	const FWorldConditionContext Context(QueryState, ContextData);
	return Context.IsTrue();
}

void FWorldConditionQuery::Deactivate(const FWorldConditionContextData& ContextData) const
{
	const FWorldConditionContext Context(QueryState, ContextData);
	return Context.Deactivate();
}

bool FWorldConditionQuery::IsActive() const
{
	return QueryState.IsInitialized();
}

#undef LOCTEXT_NAMESPACE
