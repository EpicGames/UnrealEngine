// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionSchema.h"
#include "Algo/AnyOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldConditionSchema)


TConstArrayView<FWorldConditionContextDataDesc> UWorldConditionSchema::GetContextDataDescs() const
{
	return ContextDataDescs;
}

const FWorldConditionContextDataDesc* UWorldConditionSchema::GetContextDataDescByRef(const FWorldConditionContextDataRef& Ref) const
{
	return Ref.IsValid() ? &ContextDataDescs[Ref.Index] : nullptr;
}

const FWorldConditionContextDataDesc& UWorldConditionSchema::GetContextDataDescByIndex(const int32 Index) const
{
	return ContextDataDescs[Index];
}

EWorldConditionContextDataType UWorldConditionSchema::GetContextDataTypeByRef(const FWorldConditionContextDataRef& Ref) const
{
	return Ref.IsValid() ? ContextDataDescs[Ref.Index].Type : EWorldConditionContextDataType::Persistent;
}

EWorldConditionContextDataType UWorldConditionSchema::GetContextDataTypeByIndex(const int32 Index) const
{
	return ContextDataDescs[Index].Type;
}

FWorldConditionContextDataRef UWorldConditionSchema::GetContextDataRefByName(const FName DataName, const UStruct* Struct) const
{
	const int32 Index = GetContextDataIndexByName(DataName, Struct);
	return (Index == INDEX_NONE) ? FWorldConditionContextDataRef() : FWorldConditionContextDataRef(DataName, Index);
}

int32 UWorldConditionSchema::GetContextDataIndexByName(const FName DataName, const UStruct* Struct) const
{
	return ContextDataDescs.IndexOfByPredicate([DataName, Struct](const FWorldConditionContextDataDesc& Desc)
	{
		return Desc.Name == DataName && Desc.MatchesType(Struct);
	});
}

const FWorldConditionContextDataDesc* UWorldConditionSchema::GetContextDataDescByName(const FName DataName, const TArray<const UStruct*>& Structs) const
{
	return ContextDataDescs.FindByPredicate([DataName, &Structs](const FWorldConditionContextDataDesc& Desc)
	{
		return Desc.Name == DataName && Algo::AnyOf(Structs, [&Desc] (const UStruct* Struct)
			{
				return Desc.MatchesType(Struct);
			});
	});
}

FWorldConditionContextDataRef UWorldConditionSchema::AddContextDataDesc(const FName InName, const UStruct* InStruct, const EWorldConditionContextDataType InType)
{
	const int32 Index = ContextDataDescs.Num();
	ContextDataDescs.Emplace(InName, InStruct, InType);
	return FWorldConditionContextDataRef(InName, Index);
}
