// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionsModule.h"

#define LOCTEXT_NAMESPACE "WorldConditions"

class FWorldConditionsModule : public IWorldConditionsModule
{
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FWorldConditionsModule, WorldConditions)

void FWorldConditionsModule::StartupModule()
{
}

void FWorldConditionsModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
