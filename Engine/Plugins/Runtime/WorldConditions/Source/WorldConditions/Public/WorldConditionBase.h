// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldConditionTypes.h"
#include "WorldConditionBase.generated.h"

#define UE_API WORLDCONDITIONS_API

struct FWorldConditionContext;
struct FWorldConditionQueryState;
class UWorldConditionSchema;

/**
 * Base struct for all World Conditions.
 *
 * World Conditions are used together with World Condition Query to create expressions of conditions whose result can be checked.
 * The conditions can be based on globally accessible data (e.g. subsystems), or based on any of the context data accessible via FWorldConditionContext.
 * The data that is available is defined for each use case using a UWorldConditionSchema. It also defines which conditions are available when creating
 * a query in that use case.
 *
 * FWorldConditionContextDataRef allows to reference a specific context data. When added on a world condition as property, the UI allows to pick
 * context data of specified type, and in code, pointer to the actual data can be accessed via the context. 
 *
 *		UPROPERTY(EditAnywhere, Category="Default", meta=(BaseStruct="/Script/ModuleFoo.StructBar"))
 *		FWorldConditionContextDataRef BarRef;
 *
 *		FWorldConditionResult FWorldConditionFoo::IsTrue(const FWorldConditionContext& Context) const
 *		{
 *			if (FStructBar* Bar = Context.GetMutableContextDataPtr<FStructBar>(BarRef))
 *			{
 *			}
 *			...
 *		}
 *
 * Under the hood a reference is a name that needs to be turned into an index before use. This is done on Initialize():
 *
 *		bool FWorldConditionFoo::Initialize(const UWorldConditionSchema& Schema)
 *		{
 *			if (!Schema.ResolveContextDataRef<FStructBar>(BarRef))
 *			{
 *				return false;
 *			}
 *			...
 *		}
 *
 * To speed up query evaluation, the result of a World Condition can be cached by World Condition Query.
 * A result can be cached, if the condition is based on globally accessible data, or context data that is marked as Permanent.
 * Context data marked as Dynamic may change each time IsTrue() is called and the result should never be cached.
 *
 * To indicate that a result can be cached it should have bCanBeCached set to true when returned by IsTrue().
 *
 * In cases where a data change callback can be registered based on context data, the caching can be determined
 * during Initialize() by checking of the referenced data is persistent:
 *
 *		bool FWorldConditionFoo::Initialize(const UWorldConditionSchema& Schema)
 *		{
 *			...
 *			bCanCacheResult = Schema.GetContextDataTypeByRef(BarRef) == EWorldConditionContextDataType::Persistent)
 *			...
 *		}
 *
 * The caching status is returned from IsTrue() along with the result:
 *
 *		FWorldConditionResult FWorldConditionFoo::IsTrue(const FWorldConditionContext& Context) const
 *		{
 *			FWorldConditionResult Result(EWorldConditionResult::IsFalse, bCanCacheResult);
 *			...
 *			return Result;
 *		}
 *
 * When the result is cached, it needs to be invalidated when new value arrives. This can be done using e.g. a delegate callback and invalidation handle.
 * The call to InvalidateResult() on that handle will invalidate the query state so that next time IsTrue() is called required conditions will be re-evaluated.
 * It is advised to do as little work as possible in the delegate callback:
 *
 *		bool FWorldConditionFoo::Activate(const FWorldConditionContext& Context) const
 *		{
 *			FStructBar* Bar = Context.GetMutableContextDataPtr<FStructBar>(BarRef)
 *			if (bCanCacheResult && Bar != nullptr)
 *			{
 *				if (FOnFooEvent* Delegate = Bar->GetDelegate())
 *				{
 *					FStateType& State = Context.GetState(*this);
 *					State.DelegateHandle = Delegate->AddLambda([this, InvalidationHandle = Context.GetInvalidationHandle(*this)]()
 *                      {
 *                          InvalidationHandle.InvalidateResult();
 *                      });
 *				}
 *			...
 *		}
 *
 *	Note that bCanCacheResult is stored in the condition inside world condition definition, and is the same for all instances of the world condition query state.
 *	Sometimes the caching status also relies on the input data, say, a condition may operate with a component or interface
 *	of a given actor, one of which may not have invalidation callbacks.
 *	In such case we can use some data in the condition state to decide the caching status of the result when returning it from IsTrue():
 *	
 *		FWorldConditionResult FWorldConditionFoo::IsTrue(const FWorldConditionContext& Context) const
 *		{
 *			FStateType& State = Context.GetState(*this);
 *
 *			// Only cache result if we were able to register invalidation callback
 *			const bool bResultCanBeCached = State.DelegateHandle.IsValid();
 *			FWorldConditionResult Result(EWorldConditionResult::IsFalse, bResultCanBeCached);
 *
 *			if (FStructBar* Bar = Context.GetMutableContextDataPtr<FStructBar>(BarRef))
 *			{
 *				if (Bar->EvaluateSomething())
 *				{
 *					Result.Value = EWorldConditionResult::IsTrue;
 *				}
 *			}
 *
 *			return Result;
 *		}
 */
USTRUCT(meta=(Hidden))
struct FWorldConditionBase
{
	GENERATED_BODY()

	FWorldConditionBase()
		: bIsStateObject(false)
		, bCanCacheResult(true)
		, bInvert(false)
	{
	}
	UE_API virtual ~FWorldConditionBase();

	/** @return The Instance data type of the condition. */
	virtual TObjectPtr<const UStruct>* GetRuntimeStateType() const { return nullptr; }

	/** @return Description to be shown in the UI or any debug tool. */
	UE_API virtual FText GetDescription() const;

	/**
	 * Initializes the condition to be used with a specific schema.
	 * This is called on PostLoad(), or during editing to make sure the data stays in sync with the schema.
	 * This is the place to resolve all the Context Data References,
	 * and set bCanCacheResult based on if the context data type.
	 * @param Schema The schema to initialize the condition for.
	 * @return True if init succeeded. 
	 */
	UE_API virtual bool Initialize(const UWorldConditionSchema& Schema);

	/**
	 * Called to activate the condition.
	 * The state data for the conditions can be accessed via the Context.
	 * @param Context Context that stores the context data and state of the query.
	 * @return True if activation succeeded.
	 */
	UE_API virtual bool Activate(const FWorldConditionContext& Context) const;

	/**
	 * Called to check the condition state.
	 * The state data for the conditions can be accessed via the Context.
	 * @param Context Context that stores the context data and state of the query.
	 * @return The state of the condition.
	 */
	UE_API virtual FWorldConditionResult IsTrue(const FWorldConditionContext& Context) const;

	/**
	 * Called to deactivate the condition.
	 * The state data for the conditions can be accessed via the Context.
	 * @param Context Context that stores the context data and state of the query.
	 */
	UE_API virtual void Deactivate(const FWorldConditionContext& Context) const;

	/** @return offset in state memory where the conditions state is stored. */
	uint16 GetStateDataOffset() const { return StateDataOffset; }

	/** @return condition index. */
	uint8 GetConditionIndex() const { return ConditionIndex; }

	/** @return true if the state of the condition is UObject. */
	bool IsStateObject() const { return bIsStateObject; }

	/** @return true if conditions result should be inverted. Applied by the expression evaluator. */
	bool ShouldInvertResult() const { return bInvert; }

	/** @return operator to apply for this condition in the expression evaluation. */
	EWorldConditionOperator GetOperator() const { return Operator; }

	/** @return expression depth used by the expression evaluator. */
	uint8 GetNextExpressionDepth() const { return NextExpressionDepth; }
	
protected:
	/** Used internally, Offset of the data in the State storage. */
	uint16 StateDataOffset = 0;

	/** Used internally, Index of the condition in the definition and state storage. */
	uint8 ConditionIndex = 0;

	/** Used Internally, true if the condition has Object state. */
	uint8 bIsStateObject : 1;

	/** Set to true if the result of the IsTrue() can be cached. */
	uint8 bCanCacheResult : 1;

	/** Controls whether the value of the expressions as calculated by IsTrue should be inverted. The inversion is handled by the expression evaluator. */
	UPROPERTY()
	uint8 bInvert : 1;

	/** Operator describing how the results of the condition is combined with other conditions. Not used directly, but to set up condition item in query state. */
	UPROPERTY()
	EWorldConditionOperator Operator = EWorldConditionOperator::And;

	/** Depth controlling the parenthesis of the expression. Not used directly, but to set up condition item in query state. */
	UPROPERTY()
	uint8 NextExpressionDepth = 0;

	friend struct FWorldConditionQueryDefinition;
	friend struct FWorldConditionQuerySharedDefinition;
};


/**
 * Base struct for world conditions that do not need a specific context data to work (e.g. interfaces a subsystem).
 */
USTRUCT(meta=(Hidden))
struct FWorldConditionCommonBase : public FWorldConditionBase 
{
	GENERATED_BODY()
};

/**
 * Base struct for world conditions that require just an AActor as a context data to work.
 */
USTRUCT(meta=(Hidden))
struct FWorldConditionCommonActorBase : public FWorldConditionBase
{
	GENERATED_BODY()
};

#undef UE_API
