// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldConditionQuery.h"
#include "WorldConditionSchema.h"

#define UE_API WORLDCONDITIONS_API

struct FWorldConditionBase;

/**
 * The World condition context and context data are structs that are created when we want to interact with world conditions.
 *
 * When using FWorldConditionQuery, we only need to deal with context data. The data is defined in 
 *
 *	// Create context data for our Fantastic use case.
 *	const UFantasticConditionSchema* DefaultSchema = GetDefault<UFantasticWorldConditionSchema>();
 *	FWorldConditionContextData ConditionContextData(*DefaultSchema);
 *	ConditionContextData.SetContextData(DefaultSchema->GetActorRef(), FantasticActor);
 *
 *	if (Query.IsTrue(ConditionContextData))
 *	{
 *		...
 *	}
 *
 * When managing separate state memory, world condition context is used to bind everything together:
 *
 *	FWorldConditionContext Context(*this, Definition.Preconditions, Runtime.PreconditionState, ConditionContextData);
 *	if (!Context.IsTrue())
 *	{
 *		...
 *	}
 *
 * Note: FWorldConditionContextData and FWorldConditionContext should not be stored for longer durations.
 *
 * The expected availability of the world context data is as follows:
 *
 *	- Activate:
 *		- Dynamic: not available
 *		- Persistent: must be available
 *	- IsTrue:
 *		- Dynamic: the passed data might change on each call
 *		- Persistent: available, but must check if an object is still valid
 *	- Deactivate:
 *		- Dynamic: not available
 *		- Persistent: might not be available
 *
 * When using a delegate to invalidate the query, it is advised to store weak pointer or handle to be able to unregister the delegate
 * even if the persistent data is not available.   
 */

/**
 * Container for World Condition context data for a specific schema.
 */
struct FWorldConditionContextData
{
	FWorldConditionContextData() = default;

	/** FWorldConditionContextData should not be stored for longer durations. */
	FWorldConditionContextData(const FWorldConditionContextData& Other) = delete;
	FWorldConditionContextData(FWorldConditionContextData&& Other) = delete;
	FWorldConditionContextData& operator=(const FWorldConditionContextData& Other) = delete;
	FWorldConditionContextData& operator=(FWorldConditionContextData&& Other) = delete;

	explicit FWorldConditionContextData(const UWorldConditionSchema& InSchema)
	{
		SetSchema(InSchema);
	}

	/** @return True if the context data is initialized with a schema. */
	bool IsValid() const { return Schema != nullptr; }

	/** @return True if Schema and OtherSchema are valid pointers, and the schema for the context data is child of other context. */
	bool IsSchemaChildOf(const UWorldConditionSchema* OtherSchema) const
	{
		return Schema && OtherSchema && Schema->IsA(OtherSchema->GetClass());
	}

	/** @return the schema the context data is initialized for. */
	const UWorldConditionSchema* GetSchema() const { return Schema; }

	/** Sets schema for the context data and initializes data views. */
	void SetSchema(const UWorldConditionSchema& InSchema)
	{
		Schema = &InSchema;
		Views.Init(FWorldConditionDataView(), Schema->GetContextDataDescs().Num());
		for (int32 Index = 0; Index < Views.Num(); Index++)
		{
			Views[Index] = FWorldConditionDataView(Schema->GetContextDataTypeByIndex(Index));
		}
	}

	/** Sets context data from a struct view at location specified by Ref. */
	bool SetContextData(const FWorldConditionContextDataRef& Ref, const FConstStructView StructView)
	{
		check(Schema);
		if (Ref.IsValid())
		{
			check(StructView.GetScriptStruct()->IsChildOf(Schema->GetContextDataDescByRef(Ref)->Struct));
			Views[Ref.GetIndex()] = FWorldConditionDataView(StructView, Schema->GetContextDataTypeByRef(Ref));
			return true;
		}
		return false;
	}
	
	/** Sets context data Struct at location specified by Ref. */
	template <typename T>
	typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived, bool>::Type SetContextData(const FWorldConditionContextDataRef& Ref, const T* Value)
	{
		check(Schema);
		if (Ref.IsValid())
		{
			check(TBaseStructure<T>::Get()->IsChildOf(Schema->GetContextDataDescByRef(Ref)->Struct));
			Views[Ref.GetIndex()] = FWorldConditionDataView(TBaseStructure<T>::Get(), reinterpret_cast<const uint8*>(Value), Schema->GetContextDataTypeByRef(Ref));
			return true;
		}
		return false;
	}

	/** Sets context data Object at location specified by Ref. */
	template <typename T>
	typename TEnableIf<TIsDerivedFrom<T, UObject>::IsDerived, bool>::Type SetContextData(const FWorldConditionContextDataRef& Ref, const T* Object)
	{
		check(Schema);
		if (Ref.IsValid())
		{
			check(Object == nullptr || Object->GetClass()->IsChildOf(Schema->GetContextDataDescByRef(Ref)->Struct));
			Views[Ref.GetIndex()] = FWorldConditionDataView(Object, Schema->GetContextDataTypeByRef(Ref));
			return true;
		}
		return false;
	}

	/** Sets context data Struct at location specified by Name. */
	template <typename T>
	typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived, bool>::Type SetContextData(const FName Name, const T* Value)
	{
		check(Schema);
		const int32 Index = Schema->GetContextDataIndexByName(Name, TBaseStructure<T>::Get());
		if (Index != INDEX_NONE)
		{
			check(TBaseStructure<T>::Get()->IsChildOf(Schema->GetContextDataDescByIndex(Index).Struct));
			Views[Index] = FWorldConditionDataView(TBaseStructure<T>::Get(), reinterpret_cast<const uint8*>(Value), Schema->GetContextDataTypeByIndex(Index));
			return true;
		}
		return false;
	}

	/** Sets context data Object at location specified by Name. */
	template <typename T>
	typename TEnableIf<TIsDerivedFrom<T, UObject>::IsDerived, bool>::Type SetContextData(const FName Name, const T* Object)
	{
		check(Schema);
		const int32 Index = Schema->GetContextDataIndexByName(Name, T::StaticClass());
		if (Index != INDEX_NONE)
		{
			check(Object == nullptr || Object->GetClass()->IsChildOf(Schema->GetContextDataDescByIndex(Index).Struct));
			Views[Index] = FWorldConditionDataView(Object, Schema->GetContextDataTypeByIndex(Index));
			return true;
		}
		return false;
	}

	/** @return Type of the referenced context data. */
	EWorldConditionContextDataType GetContextDataType(const FWorldConditionContextDataRef& Ref) const
	{
		check(Ref.IsValid());
		return Views[Ref.GetIndex()].GetType();
	}

	/** @return Pointer to referenced context data. */
	template <typename T>
	T* GetContextDataPtr(const FWorldConditionContextDataRef& Ref) const
	{
		check(Ref.IsValid());
		return Views[Ref.GetIndex()].template GetPtr<T>();
	}

	/** @return Pointer to referenced context data if the type is correct, else returns nullptr. */
	template <typename T>
	T* TryGetContextDataPtr(const FWorldConditionContextDataRef& Ref) const
	{
		check(Ref.IsValid());
		return Views[Ref.GetIndex()].template TryGetPtr<T>();
	}

protected:
	/** Pointer to schema used to initialize the context data. */
	const UWorldConditionSchema* Schema = nullptr;
	
	/** Views to context data. */
	TArray<FWorldConditionDataView> Views;
};

/**
 * The World Condition context is used to activate, update, and deactivate a world condition.
 * It ties together the context data, query definition, and query state, and allows data access for the query conditions.
 */
struct FWorldConditionContext
{
	explicit FWorldConditionContext(FWorldConditionQueryState& InQueryState, const FWorldConditionContextData& InContextData)
		: QueryState(InQueryState)
		, ContextData(InContextData)
	{
		World = IsValid(QueryState.GetOwner()) ? QueryState.GetOwner()->GetWorld() : nullptr;
	}

	/** FWorldConditionContext should not be stored for longer durations. */
	FWorldConditionContext(const FWorldConditionContext& Other) = delete;
	FWorldConditionContext(FWorldConditionContext&& Other) = delete;
	FWorldConditionContext& operator=(const FWorldConditionContext& Other) = delete;
	FWorldConditionContext& operator=(FWorldConditionContext&& Other) = delete;

	/** @return Pointer to owner of the world conditions to be updated. */
	const UObject* GetOwner() const { return QueryState.GetOwner(); }
	
	/** @return Pointer to world of the owner of the world conditions to be updated. */
	UWorld* GetWorld() const { return World; }
	
	/** @return Pointer to the schema of the context data passed to the conditions. */
	const UWorldConditionSchema* GetSchema() const { return ContextData.GetSchema(); }

	/** @return Type of the referenced context data. */
	EWorldConditionContextDataType GetContextDataType(const FWorldConditionContextDataRef& Ref) const
	{
		return ContextData.GetContextDataType(Ref);
	}

	/** @return Pointer to referenced context data. */
	template <typename T>
	T* GetContextDataPtr(const FWorldConditionContextDataRef& Ref) const
	{
		return ContextData.template GetContextDataPtr<T>(Ref);
	}

	/**
	 * Usage Example : auto [Actor, Position] = Context.GetContextDataTuplePtr<AActor, FVector>(Ref);
	 * @return a tuple of pointers of the given types to the property if possible, nullptr otherwise. 
	 */
	template <typename ...T>
	TTuple<T*...> GetContextDataTuplePtr(const FWorldConditionContextDataRef& Ref) const
	{
		return TTuple<T*...>(ContextData.template TryGetContextDataPtr<T>(Ref)...);
	}

	/** @return Struct State data of the specific world condition. */
	template <typename T>
	typename T::FStateType& GetState(const T& Condition) const
	{
		static_assert(TIsDerivedFrom<T, FWorldConditionBase>::IsDerived, "Expecting Conditions to derive from FWorldConditionBase.");
		return QueryState.GetStateStruct(Condition).template Get<typename T::FStateType>();
	}

	/** @return Object State data of the specific world condition. */
	template <typename T>
	typename T::UStateType& GetState(const T& Condition) const
	{
		static_assert(TIsDerivedFrom<T, FWorldConditionBase>::IsDerived, "Expecting Conditions to derive from FWorldConditionBase.");
		return *CastChecked<typename T::UStateType>(QueryState.GetStateObject(Condition));
	}

	/** @return Reference to the query state of update world condition query. */
	FWorldConditionQueryState& GetQueryState() const { return QueryState; }

	/**
	 * Returns handle that can be used to invalidate specific condition and recalculate the condition.
	 * The handle can be acquired via FWorldConditionContext or FWorldConditionQueryState
	 * and is guaranteed to be valid while the query is active.
	 * @return Invalidation handle.	
	 */
	FWorldConditionResultInvalidationHandle GetInvalidationHandle(const FWorldConditionBase& Condition) const
	{
		return QueryState.GetInvalidationHandle(Condition);
	}

	/**
	 * Calls Activate() on the world conditions in the query.
	 * @return true of the activation succeeded, or false if failed. Failed queries will return false when IsTrue() is called.
	 */
	UE_API bool Activate() const;

	/**
	 * Evaluates the result of the query.
	 * Intermediate results may be cached, and are stored in the query state.
	 * If a cached result is invalidated, or the query relies on dynamic context data, IsTrue() is called on the necessary conditions.
	 * @return the value of the query condition expression.
	 */
	UE_API bool IsTrue() const;

	/**
	 * Calls Deactivate() on the world conditions in the query.
	 */
	UE_API void Deactivate() const;
	
protected:
	/** World of the QueryState.Owner. */
	UWorld* World = nullptr;
	
	/** Reference to the query state of the query to be updated. */
	FWorldConditionQueryState& QueryState;
	
	/** Reference to the context data for the query to be updated. */
	const FWorldConditionContextData& ContextData;
};

#undef UE_API
