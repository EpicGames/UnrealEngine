// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStructContainer.h"
#include "Templates/SubclassOf.h"
#include "WorldConditionBase.h"
#include "WorldConditionSchema.h"
#include "WorldConditionQuery.generated.h"

#define UE_API WORLDCONDITIONS_API

struct FPropertyBindingPath;
struct FWorldConditionContext;
struct FWorldConditionContextData;
class FReferenceCollector;

/**
 * World Condition Query is an expression of World Conditions whose state can be queried.
 * The state of query and individual conditions can be cached, which allows to evaluate the conditions quickly.
 * See FWorldConditionBase for more information about the conditions.
 *
 * The World Condition Query is split it two parts: FWorldConditionQueryDefinition and FWorldConditionQueryState.
 * Definition is the "const" part of the query and state contain runtime caching and runtime state of the condition.
 * This allows the definition to be stored in an asset, and we can allocate just the per instance data when needed.
 *
 * Conditions operate on context data which is defined in a UWorldConditionSchema. The schema describes what kind
 * of structs and objects are available as input for the conditions, and what conditions can be used in specific use case.
 *
 * The state is tightly coupled to the definition. The memory layout of the state is stored in the definition.
 *
 * For convenience there is also FWorldConditionQuery which combines these two in one package.
 */

/**
 * Struct used to store a world condition in editor or to apply hotfixes. Used internally.
 * Note that the Operator and ExpressionDepth are stored here separately from the World Condition to make sure they are not reset if the Condition is empty. 
 */
USTRUCT()
struct FWorldConditionEditable
{
	GENERATED_BODY()

	FWorldConditionEditable() = default;

	FWorldConditionEditable(const uint8 InExpressionDepth, const EWorldConditionOperator InOperator, const FConstStructView InCondition)
		: ExpressionDepth(InExpressionDepth)
		, Operator(InOperator)
		, Condition(InCondition)
	{
	}

	FWorldConditionEditable(const uint8 InExpressionDepth, const EWorldConditionOperator InOperator, const bool bInInvert, const FConstStructView InCondition)
		: ExpressionDepth(InExpressionDepth)
		, Operator(InOperator)
		, bInvert(bInInvert)
		, Condition(InCondition)
	{
	}

	void Reset()
	{
		Operator = EWorldConditionOperator::And;
		ExpressionDepth = 0;
		Condition.Reset();
	}

	bool operator==(const FWorldConditionEditable& Other) const
	{
		return ExpressionDepth == Other.ExpressionDepth
				&& Operator == Other.Operator
				&& bInvert == Other.bInvert
				&& Condition == Other.Condition;
	}

	bool operator!=(const FWorldConditionEditable& Other) const
	{
		return !(*this == Other);
	}

	/** Expression depth controlling the parenthesis of the expression. */
	UPROPERTY(EditAnywhere, Category="Default")
	uint8 ExpressionDepth = 0;

	/** Operator describing how the results of the condition is combined with other conditions. */
	UPROPERTY(EditAnywhere, Category="Default")
	EWorldConditionOperator Operator = EWorldConditionOperator::And;

	/** Controls whether the value of the expressions as calculated by IsTrue should be inverted. */
	UPROPERTY(EditAnywhere, Category="Default")
	bool bInvert = false;

	/** Instance of a world condition. */
	UPROPERTY(EditAnywhere, Category="Default")
	FInstancedStruct Condition;
};

/**
 * Class that describes a specific configuration of a world condition. Should not be used directly.
 * It is shared between all query states initialized with a specific FWorldConditionQueryDefinition.
 */
USTRUCT()
struct FWorldConditionQuerySharedDefinition
{
	GENERATED_BODY()
	
	FWorldConditionQuerySharedDefinition()
		: bIsLinked(false)
	{
	}

	UE_API void PostSerialize(const FArchive& Ar);
	UE_API bool Identical(const FWorldConditionQuerySharedDefinition* Other, uint32 PortFlags) const;

	/** @return the schema used for the definition. */
	TSubclassOf<UWorldConditionSchema> GetSchemaClass() const
	{
		return SchemaClass;
	}

	/** @return conditions in the definition. */
	const FInstancedStructContainer& GetConditions() const
	{
		return Conditions;
	}

	/** @return the amount of memory needed for the state (in bytes). */
	int32 GetStateSize() const
	{
		return StateSize;
	}
	
	/** @return the alignment of the memory needed for the state (in bytes). */
	int32 GetStateMinAlignment() const
	{
		return StateMinAlignment;
	}

	/** @return true if the shared definition is successfully linked. */
	bool IsLinked() const
	{
		return bIsLinked;
	}
	
private:

	/** Updates the state memory layout and initializes the conditions. */
	UE_API bool Link(const UObject* Outer);
	
	/** Sets the definition data. */
	UE_API void Set(const TSubclassOf<UWorldConditionSchema> InSchema, const TArrayView<FConstStructView> InConditions);
	UE_API void Set(const TSubclassOf<UWorldConditionSchema> InSchema, const TArrayView<FStructView> InConditions);
	
	/** All the conditions of the world conditions. */
	UPROPERTY()
	FInstancedStructContainer Conditions;

	/** Schema used to create the conditions. */
	UPROPERTY()
	TSubclassOf<UWorldConditionSchema> SchemaClass;

	/** Min alignment of the state storage */
	int32 StateMinAlignment = 8;
	
	/** Size of the state storage */
	int32 StateSize = 0;

	/** Flag indicating the last result of Link(), if true, the definition is ready to use. */
	uint8 bIsLinked : 1;
	
	friend struct FWorldConditionQueryDefinition;
};

template<>
struct TStructOpsTypeTraits<FWorldConditionQuerySharedDefinition> : public TStructOpsTypeTraitsBase2<FWorldConditionQuerySharedDefinition>
{
	enum
	{
		WithPostSerialize = true,
		WithIdentical = true,
	};
};

/**
 * Definition of a world condition.
 * The mutable state of the world condition is stored in FWorldConditionQueryState.
 * This allows to reuse the definitions and minimize the runtime memory needed to run queries.
 */
USTRUCT(BlueprintType)
struct FWorldConditionQueryDefinition
{
	GENERATED_BODY()

	/** Sets the schema class that is used to create the conditions. */
	UE_API void SetSchemaClass(const TSubclassOf<UWorldConditionSchema> InSchema);

	/** @return the schema class of the definition. */
	TSubclassOf<UWorldConditionSchema> GetSchemaClass() const
	{
		return SchemaClass;
	}
	
	/** @return true of the definition has conditions and has been initialized. */
	UE_API bool IsValid() const;

	enum class EInitializationMode : uint8
	{
		/**
		 * The Initialize function will create a SharedDefinition only if there are Editable conditions, otherwise it will be invalid.
		 */
		ResetSharedDefinitionOnEmptyConditions,

		/**
		 * The Initialize function will preserve the existing SharedDefinition if there are no Editable conditions.
		 */
		UseExistingSharedDefinitionOnEmptyConditions,

		/**
		 * The default initialization mode is target specific:
		 * 
		 * - In Editor builds:
		 *   Editable conditions are serialized and represent the source data used to generate the runtime conditions inside the shared definition.
		 *   SharedDefinition existence is entirely controlled by those conditions, so we use 'ResetSharedDefinitionOnEmptyConditions'.
		 *
		 * - In cooked builds:
		 *   Editable conditions are not serialized in the cooked data so it is expected that the list of Editable conditions
		 *   will be empty when external systems are calling Initialize, so we use 'UseExistingSharedDefinitionOnEmptyConditions'.
		 */
#if WITH_EDITORONLY_DATA
		Default = ResetSharedDefinitionOnEmptyConditions
#else
		Default = UseExistingSharedDefinitionOnEmptyConditions
#endif
	};

	/**
	 * Initialized the condition from editable data.
	 * @param Outer Used in logging error messages, optional
	 * @param InitializationMode Indicates how the method will handle an empty list of Editable conditions
	 * @return true if initialization succeeded.
	 */
	UE_API bool Initialize(const UObject* Outer, EInitializationMode InitializationMode = EInitializationMode::Default);

	UE_API bool Serialize(FArchive& Ar);
	UE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive = nullptr);
	UE_API bool ExportTextItem(FString& ValueStr, FWorldConditionQueryDefinition const& DefaultValue, class UObject* Parent, int32 PortFlags, class UObject* ExportRootScope) const;
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector) const;
	UE_API bool Identical(const FWorldConditionQueryDefinition* Other, uint32 PortFlags) const;

#if WITH_EDITORONLY_DATA
	/** Initialized the condition with specific data. */
	UE_API bool Initialize(const UObject* Outer, const TSubclassOf<UWorldConditionSchema> InSchemaClass, const TConstArrayView<FWorldConditionEditable> InConditions);

	/** Adds a single condition to the definition. The condition will not be in use until Initialize() is called, and a state is initialized. */
	void AddCondition(const FWorldConditionEditable& NewCondition)
	{
		EditableConditions.Add(NewCondition);
	}
#endif

	/** @returns description of the query expression. */
	UE_API FText GetDescription() const;

	FWorldConditionQuerySharedDefinition* GetSharedDefinition() const
	{
		return SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	}

	FSharedStruct GetSharedDefinitionStruct() const
	{
		return SharedDefinition;
	}

	static UE_API bool TryRetargetingPathToConditions(FPropertyBindingPath& InOutPath);

private:
	/**
	 * The definition used to initialize and execute world conditions.
	 * Created from editable conditions during edit via Initialize().
	 */
	UPROPERTY(transient)
	FSharedStruct SharedDefinition;

	/** Schema of the definition, also stored in SharedDefinition. */
	UPROPERTY()
	TSubclassOf<UWorldConditionSchema> SchemaClass;

	/**
	 * Conditions used while editing, converted in to Conditions via Initialize().
	 * Those will be cleared for cooked builds but can still be used by hotfixes.
	 */
	UPROPERTY(EditAnywhere, Category="Default", meta=(BaseStruct = "/Script/SmartObjectsModule.WorldCondition"))
	TArray<FWorldConditionEditable> EditableConditions;
};

template<>
struct TStructOpsTypeTraits<FWorldConditionQueryDefinition> : public TStructOpsTypeTraitsBase2<FWorldConditionQueryDefinition>
{
	enum
	{
		WithCopy = true, // Ensures that the SharedDefinition gets copied too.
		WithIdentical = true,
		WithAddStructReferencedObjects = true,
		WithSerializer = true,
		WithImportTextItem = true,
		WithExportTextItem = true,
	};
};


/**
 * Item used to describe the structure of a world condition query for fast traversal of the expression.
 */
USTRUCT()
struct FWorldConditionItem
{
	GENERATED_BODY()

	FWorldConditionItem() = default;
	
	explicit FWorldConditionItem(const EWorldConditionOperator InOperator, const uint8 InNextExpressionDepth)
		: Operator(InOperator)
		, NextExpressionDepth(InNextExpressionDepth)
	{
	}

	/** Operator describing how the results of the condition is combined with other conditions. */
	EWorldConditionOperator Operator = EWorldConditionOperator::And;
	
	/** Expression depth controlling the parenthesis of the expression. */
	uint8 NextExpressionDepth = 0;
	
	/** Cached result of the condition. */
	EWorldConditionResultValue CachedResult = EWorldConditionResultValue::Invalid;
};

/**
 * Struct used to store the pointer to an UObject based condition state.
 */
USTRUCT()
struct FWorldConditionStateObject
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UObject> Object;
};


/**
 * Handle that can be used to invalidate the cached result of a condition (and query).
 * The handle can be acquired via FWorldConditionContext or FWorldConditionQueryState
 * and is guaranteed to be valid while the query is active.
 */
struct FWorldConditionResultInvalidationHandle
{
	FWorldConditionResultInvalidationHandle() = default;

	/** @return True if the handle is valid. */
	bool IsValid() const { return WeakStateMemory.IsValid(); }

	/** Resets the handle. */
	void Reset()
	{
		WeakStateMemory = nullptr;
		ItemOffset = 0;
	}

	/**
	 * When called invalidates the result of the query and condition. 
	 */
	UE_API void InvalidateResult() const;
	
protected:

	explicit FWorldConditionResultInvalidationHandle(const TWeakPtr<uint8>& InStateMemory, const int32 InItemOffset)
		: WeakStateMemory(InStateMemory)
		, ItemOffset(InItemOffset)
	{
	}

	TWeakPtr<uint8> WeakStateMemory = nullptr;
	int32 ItemOffset = 0;

	friend struct FWorldConditionQueryState;
};

/**
 * Runtime state of a world conditions.
 * The structure of the data for the state is defined in a query definition.
 * The definition and conditions are stored in FWorldConditionQueryDefinition.
 * This allows to reuse the definitions and minimize the runtime memory needed to run queries.
 *
 * Note: Any code embedding this struct is responsible for calling AddReferencedObjects().
 */
USTRUCT()
struct FWorldConditionQueryState
{
	GENERATED_BODY()

	FWorldConditionQueryState()
		: bIsInitialized(false)
		, bAreConditionsActivated(false)
	{
	}

	FWorldConditionQueryState(const FWorldConditionQueryState& Other)
		: FWorldConditionQueryState()
	{
		// Query state cannot be copied, it always needs to be initialized for specific purpose. 
	}

	FWorldConditionQueryState(FWorldConditionQueryState&& Other)
		: NumConditions(Other.NumConditions)
		, bIsInitialized(Other.bIsInitialized)
		, bAreConditionsActivated(Other.bAreConditionsActivated)
		, Memory(MoveTemp(Other.Memory))
		, SharedDefinition(MoveTemp(Other.SharedDefinition))
		, Owner(MoveTemp(Other.Owner))
	{
	}

	FWorldConditionQueryState& operator=(FWorldConditionQueryState&& Other)
	{
		if (this == &Other)
		{
			return *this;
		}
		NumConditions = Other.NumConditions;
		bIsInitialized = Other.bIsInitialized;
		bAreConditionsActivated = Other.bAreConditionsActivated;
		Memory = MoveTemp(Other.Memory);
		SharedDefinition = MoveTemp(Other.SharedDefinition);
		Owner = MoveTemp(Other.Owner);
		return *this;
	}

	UE_API ~FWorldConditionQueryState();

	FWorldConditionQueryState& operator=(const FWorldConditionQueryState& RHS)
	{
		// Query state cannot be copied, it always needs to be initialized for specific purpose. 
		return *this;
	}

	/** @return True if the state is properly initialized. */
	bool IsInitialized() const
	{
		return bIsInitialized;
	}

	/**
	 * Initialized the state for a specific query definition.
	 * @param Owner Owner of any objects created during Init().
	 * @param QueryDefinition definition of the state to initialize.
	 */
	UE_API void Initialize(const UObject& Owner, const FWorldConditionQueryDefinition& QueryDefinition);

	/**
	 * Frees the allocated data and objects. The definition must be the same as used in init
	 * as it is used to traverse the structure in memory.
	 */
	UE_API void Free();

	/** @return True if the conditions were activated. */
	bool AreConditionsActivated() const
	{
		return bAreConditionsActivated;
	}

	/** Sets the status of the conditions for this state. */
	void SetConditionsActivated(const bool bConditionsActivated)
	{
		bAreConditionsActivated = bConditionsActivated;
	}

	/** @return cached result stored in the state. */
	EWorldConditionResultValue GetCachedResult() const
	{
		check(bIsInitialized);
		// If Memory is null, and the query is initialized, it's empty query, which evaluates to true. 
		return Memory.IsValid() ? *reinterpret_cast<EWorldConditionResultValue*>(Memory.Get() + CachedResultOffset) : EWorldConditionResultValue::IsTrue;
	}

	/** @return Condition item at specific index */
	FWorldConditionItem& GetItem(const int32 Index) const
	{
		check(bIsInitialized);
		check(Memory.IsValid() && Index >= 0 && Index < static_cast<int32>(NumConditions));
		return *reinterpret_cast<FWorldConditionItem*>(Memory.Get() + ItemsOffset + Index * sizeof(FWorldConditionItem));
	}

	/** @return Object describing the state of a specified condition. */
	UObject* GetStateObject(const FWorldConditionBase& Condition) const
	{
		check(bIsInitialized);
		check(Condition.GetStateDataOffset() > 0);
		check(Condition.IsStateObject());
		check(Memory.IsValid());
		const FWorldConditionStateObject& StateObject = *reinterpret_cast<FWorldConditionStateObject*>(Memory.Get() + Condition.GetStateDataOffset());
		return StateObject.Object;
	}

	/** @return struct describing the state of a specified condition. */
	FStructView GetStateStruct(const FWorldConditionBase& Condition) const
	{
		check(bIsInitialized);
		check(Condition.GetStateDataOffset() > 0);
		check (!Condition.IsStateObject());
		const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Condition.GetRuntimeStateType()->Get());
		check(ScriptStruct);
		check(Memory.IsValid());
		return FStructView(Cast<UScriptStruct>(Condition.GetRuntimeStateType()->Get()), Memory.Get() + Condition.GetStateDataOffset());
	}

	/** @return The number of conditions in the state data. */
	int32 GetNumConditions() const { return NumConditions; }

	/** Adds referenced objects to the collector. */
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);

	/**
	 * Returns handle that can be used to invalidate specific condition and recalculate the condition.
	 * The handle can be acquired via FWorldConditionContext or FWorldConditionQueryState
	 * and is guaranteed to be valid while the query is active.
	 * @return Invalidation handle.	
	 */
	UE_API FWorldConditionResultInvalidationHandle GetInvalidationHandle(const FWorldConditionBase& Condition) const;

	/** @return the owner the state was initialized with. */
	const UObject* GetOwner() const { return Owner; }

	/** @return shared definition used to initialize the state, or nullptr if query is empty. */
	const FWorldConditionQuerySharedDefinition* GetSharedDefinition() const
	{
		return SharedDefinition.GetPtr<FWorldConditionQuerySharedDefinition>();
	}

	/** Sets the combined cached result of the query. */
	void SetCachedResult(const EWorldConditionResultValue InResult) const
	{
		check(bIsInitialized);
		if (Memory.IsValid())
		{
			EWorldConditionResultValue& CachedResult = *reinterpret_cast<EWorldConditionResultValue*>(Memory.Get() + CachedResultOffset); 
			CachedResult = InResult;
		}
	}

	/** Offset in state memory where cached result is. */
	static constexpr int32 CachedResultOffset = 0;

	/** Offset in state memory where condition items are. */
	static constexpr int32 ItemsOffset = static_cast<int32>(Align(sizeof(EWorldConditionResultValue), alignof(FWorldConditionItem)));
 
private:

	UE_API void InitializeInternal(const UObject* InOwner, const FSharedStruct InSharedDefinition);

	uint8 NumConditions = 0;
	uint8 bIsInitialized : 1;
	uint8 bAreConditionsActivated : 1;
	TSharedPtr<uint8> Memory = nullptr;
	FSharedStruct SharedDefinition;

	UPROPERTY(Transient)
	TObjectPtr<const UObject> Owner = nullptr;
};

template<>
struct TStructOpsTypeTraits<FWorldConditionQueryState> : public TStructOpsTypeTraitsBase2<FWorldConditionQueryState>
{
	enum
	{
		WithAddStructReferencedObjects = true,
	};
};


/**
 * General purpose World Condition Query that combines Query Definition and Query State in one.
 */
USTRUCT()
struct FWorldConditionQuery
{
	GENERATED_BODY()

	/** @return True if the query is activated. */
	UE_API bool IsActive() const;

	/**
	 * Activates the world conditions in the query.
	 * @param Owner Owner of any objects created during Activate().
	 * @param ContextData ContextData that matches the schema of the query.
	 * @return true of the activation succeeded, or false if failed. Failed queries will return false when IsTrue() is called.
	 */
	UE_API bool Activate(const UObject& Owner, const FWorldConditionContextData& ContextData) const;

	/**
	 * Returns the result of the query. Cached state is returned if it is available,
	 * if update is needed or the query has dynamic context data, IsTrue() is called on the necessary conditions.
	 * @param ContextData ContextData that matches the schema of the query.
	 * @return the value of the query condition expression.
	 */
	UE_API bool IsTrue(const FWorldConditionContextData& ContextData) const;
	
	/**
	 * Deactivates the world conditions in the query.
	 * @param ContextData ContextData that matches the schema of the query.
	 */
	UE_API void Deactivate(const FWorldConditionContextData& ContextData) const;

#if WITH_EDITORONLY_DATA
	/**
	 * Initializes a query from array of conditions for testing.
	 * @return true of the query was created and initialized.
	 */
	UE_API bool DebugInitialize(const UObject* Outer, const TSubclassOf<UWorldConditionSchema> InSchemaClass, const TConstArrayView<FWorldConditionEditable> InConditions);
#endif // WITH_EDITORONLY_DATA
	
protected:
	/** Defines the conditions to run on the query.  */
	UPROPERTY(EditAnywhere, Category="Default");
	FWorldConditionQueryDefinition QueryDefinition;

	/** Runtime state of the query. */
	UPROPERTY(Transient);
	mutable FWorldConditionQueryState QueryState;
};

#undef UE_API
