// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldConditionTypes.h"
#include "WorldConditionSchema.generated.h"

#define UE_API WORLDCONDITIONS_API

/**
 * Describes the context data and allowed world conditions for a specific use case.
 *
 * Each schema adds the context data they can provide in their class constructor. This allows
 * the derived classes to add more data as needed.
 *
 *	UCLASS()
 *	class UFantasticWorldConditionSchema : public UWorldConditionSchema
 *	{
 *		GENERATED_BODY()
 *	public:
 *		UFantasticWorldConditionSchema::UFantasticWorldConditionSchema(const FObjectInitializer& ObjectInitializer)
 *			: Super(ObjectInitializer)
 *		{
 *			ActorRef = AddContextDataDesc(TEXT("Actor"), AActor::StaticClass(), EWorldConditionContextDataType::Dynamic);
 *		}
 *
 *		// For convenience, when the schema is known, makes it easy to set the context data.
 *		FWorldConditionContextDataRef GetActorRef() const { return ActorRef; }
 *
 *	protected:
 *		// Filter condition classes that make sense for this context.
 *		virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override;
 *		{
 *			return Super::IsStructAllowed(InScriptStruct)
 *				|| InScriptStruct->IsChildOf(TBaseStructure<FFantasticWorldConditionBase>::Get());
 * 		}
 *		
 *		FWorldConditionContextDataRef ActorRef;
 *	};
*/


UCLASS(MinimalAPI)
class UWorldConditionSchema : public UObject
{
	GENERATED_BODY()

public:

	/** @return True if world condition of the specific type is allowed. */
	virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const { return false; }

	/** @return All condition descriptors. */
	UE_API TConstArrayView<FWorldConditionContextDataDesc> GetContextDataDescs() const;

	/** @return Context data descriptor of specific name and types. */
	UE_API const FWorldConditionContextDataDesc* GetContextDataDescByName(const FName DataName, const TArray<const UStruct*>& Structs) const;

	/** @return Context data descriptor of specific name and type. */
	UE_API const FWorldConditionContextDataDesc* GetContextDataDescByRef(const FWorldConditionContextDataRef& Ref) const;

	/** @return Context data descriptor of specified index. */
	UE_API const FWorldConditionContextDataDesc& GetContextDataDescByIndex(const int32 Index) const;

	/** @return Context data type based on reference. */
	UE_API EWorldConditionContextDataType GetContextDataTypeByRef(const FWorldConditionContextDataRef& Ref) const;

	/** @return Context data type based on index. */
	UE_API EWorldConditionContextDataType GetContextDataTypeByIndex(const int32 Index) const;

	/** @return Context data reference of specific name and type. */
	UE_API FWorldConditionContextDataRef GetContextDataRefByName(const FName DataName, const UStruct* Struct) const;
	
	/** @return Context data index by name. */
	UE_API int32 GetContextDataIndexByName(const FName DataName, const UStruct* Struct) const;

	/**
	 * Resolves the index of the context data reference to an object data, based on the name in the reference and the types from the template. 
	 * Usage example : Schema.ResolveContextDataRef<AActor, FVector>(SourceWorldConditionContextDataRef)
	 * @param Ref Reference to resolve.
	 * @return True if the reference was resolved.
	 */
	template <typename Head, typename NextHead, typename... Tail>
	typename TEnableIf<TIsDerivedFrom<Head, UObject>::IsDerived, bool>::Type ResolveContextDataRef(FWorldConditionContextDataRef& Ref) const
	{
		const int32 Index = GetContextDataIndexByName(Ref.Name, Head::StaticClass());
		if (Index == INDEX_NONE)
		{
			return ResolveContextDataRef<NextHead, Tail...>(Ref);
		}

		Ref.Index = IntCastChecked<uint8>(Index);
		return true;
	}

	/**
	 * Resolves the index of the context data reference to an object data, based on the name in the reference and the type from the template. 
	 * Usage example : Schema.ResolveContextDataRef<AActor>(SourceActorWorldConditionContextDataRef)
	 * @param Ref Reference to resolve.
	 * @return True if the reference was resolved.
	 */
	template <typename Head>
	typename TEnableIf<TIsDerivedFrom<Head, UObject>::IsDerived, bool>::Type ResolveContextDataRef(FWorldConditionContextDataRef& Ref) const
	{
		const int32 Index = GetContextDataIndexByName(Ref.Name, Head::StaticClass());
		if (Index == INDEX_NONE)
		{
			Ref.Index = FWorldConditionContextDataRef::InvalidIndex;
			return false;
		}

		Ref.Index = IntCastChecked<uint8>(Index);
		return true;
	}

	/**
	 * Resolves the index of the context data reference to a Struct context data, based on the name in the reference and the types from the template. 
	 * Usage example : Schema.ResolveContextDataRef<AActor, FVector>(SourceWorldConditionContextDataRef)
	 * @param Ref Reference to resolve.
	 * @return True if the reference was resolved.
	 */
	template <typename Head, typename NextHead, typename... Tail>
	typename TEnableIf<!TIsDerivedFrom<Head, UObject>::IsDerived, bool>::Type ResolveContextDataRef(FWorldConditionContextDataRef& Ref) const
	{
		const int32 Index = GetContextDataIndexByName(Ref.Name, TBaseStructure<Head>::Get());
		if (Index == INDEX_NONE)
		{
			return ResolveContextDataRef<NextHead, Tail...>(Ref);
		}
		Ref.Index = IntCastChecked<uint8>(Index);
		return true;
	}

	/**
	 * Resolves the index of the context data reference to a Struct context data, based on the name in the reference and the type from the template. 
	 * Usage example : Schema.ResolveContextDataRef<AActor>(SourceActorWorldConditionContextDataRef)
	 * @param Ref Reference to resolve.
	 * @return True if the reference was resolved.
	 */
	template <typename Head>
	typename TEnableIf<!TIsDerivedFrom<Head, UObject>::IsDerived, bool>::Type ResolveContextDataRef(FWorldConditionContextDataRef& Ref) const
	{
		const int32 Index = GetContextDataIndexByName(Ref.Name, TBaseStructure<Head>::Get());
		if (Index == INDEX_NONE)
		{
			Ref.Index = FWorldConditionContextDataRef::InvalidIndex;
			return false;
		}
		Ref.Index = IntCastChecked<uint8>(Index);
		return true;
	}

protected:

	/** Adds a context data descriptor and returns reference to it. */
	UE_API FWorldConditionContextDataRef AddContextDataDesc(const FName InName, const UStruct* InStruct, const EWorldConditionContextDataType InType);

	/** All context data descriptors. */
	UPROPERTY()
	TArray<FWorldConditionContextDataDesc> ContextDataDescs;
};

#undef UE_API
