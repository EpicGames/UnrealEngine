// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructView.h"
#include "WorldConditionTypes.generated.h"

WORLDCONDITIONS_API DECLARE_LOG_CATEGORY_EXTERN(LogWorldCondition, Warning, All);

/** Result of a world condition */
UENUM()
enum class EWorldConditionResultValue : uint8
{
	/** The result is false. */
	IsFalse = 0,
	/** The result is true. */
	IsTrue = 1,
	/** The result needs to be recalculated. */
	Invalid = 2,
};

/**
 * Result of a world condition evaluation.
 * Also contains additional information related to that result (e.g. if the result can be cached or not)
 */
USTRUCT()
struct FWorldConditionResult
{
	GENERATED_BODY()

	FWorldConditionResult() = default;
	FWorldConditionResult(const EWorldConditionResultValue Result, const bool bCanResultBeCached)
		: Value(Result)
		, bCanBeCached(bCanResultBeCached)
	{
	}

	/** Indicates the result of a condition evaluated for a given context. */
	UPROPERTY()
	EWorldConditionResultValue Value = EWorldConditionResultValue::Invalid;

	/** Indicates that a given result can be cached or not. */
	UPROPERTY()
	bool bCanBeCached = false;
};

/** Boolean operator between conditions in a expression. */
UENUM()
enum class EWorldConditionOperator : uint8
{
	/** Boolean And */
	And,
	/** Boolean Or */
	Or,
	/** Used internally, signifies that a right value is copied over left value. */
	Copy UMETA(Hidden),
};

namespace UE::WorldCondition
{
	/** Maximum depth of sub-expressions in an expression. */
	constexpr int32 MaxExpressionDepth = 4;

	/** Merges two condition results using specific operator. */
	inline EWorldConditionResultValue MergeResults(const EWorldConditionOperator Operator, const EWorldConditionResultValue InLeft, const EWorldConditionResultValue InRight)
	{
		const uint8 Left = (uint8)InLeft & 1;
		const uint8 Right = (uint8)InRight & 1;
		const uint8 ResultAnd = Left & Right;
		const uint8 ResultOr = Left | Right;
		const uint8 Result = (Operator == EWorldConditionOperator::And) ? ResultAnd : ResultOr;
		return (Operator == EWorldConditionOperator::Copy) ? InRight : (EWorldConditionResultValue)Result;
	}

	/** Inverts the result if bInvert is true. Invalid is kept intact. */
	inline EWorldConditionResultValue Invert(const EWorldConditionResultValue InResult, const bool bInvert)
	{
		if (InResult == EWorldConditionResultValue::Invalid)
		{
			return EWorldConditionResultValue::Invalid;
		}
		return ((InResult == EWorldConditionResultValue::IsTrue) ^ bInvert) ? EWorldConditionResultValue::IsTrue : EWorldConditionResultValue::IsFalse;
	}

	inline EWorldConditionResultValue FromOptional(const TOptional<bool>& InOptionalBool)
	{
		if (!InOptionalBool.IsSet())
		{
			return EWorldConditionResultValue::Invalid;
		}
		return InOptionalBool.GetValue() ? EWorldConditionResultValue::IsTrue : EWorldConditionResultValue::IsFalse;
	}

} // UE::WorldCondition

/**
  * Describes the availability of a context data.
 */
UENUM()
enum class EWorldConditionContextDataType : uint8
{
	/** The context data might change on each call, must check for validity, and is only available on IsTrue(). */
	Dynamic,
	/** The context data is the same during the lifetime of the condition, must check for validity on Activate()/IsTrue(), might not be available on Deactivate(). */
	Persistent,
};


/**
 * Describes context data available for a world conditions.
 * The descriptors are defined in UWorldConditionSchema, and context data is referred using FWorldConditionContextDataRef.
 */
USTRUCT()
struct FWorldConditionContextDataDesc
{
	GENERATED_BODY()

	FWorldConditionContextDataDesc() = default;
	explicit FWorldConditionContextDataDesc(const FName InName, const UStruct* InStruct, const EWorldConditionContextDataType InType)
		: Struct(InStruct)
		, Name(InName)
		, Type(InType)
	{
	}

	bool MatchesType(const UStruct* TypeToMatch) const
	{
		if (!Struct)
		{
			return false;
		}

		if (Struct->IsChildOf(TypeToMatch))
		{
			return true;
		}

		if (const UClass* DescClass = Cast<UClass>(Struct))
		{
			return DescClass->ImplementsInterface(Cast<UClass>(TypeToMatch));
		}

		return false;
	}

	/** Type of the context data, can be a struct or object. */
	UPROPERTY()
	TObjectPtr<const UStruct> Struct;

	/** Name of the context data, which is used together with the type struct to find specific data to use. */
	UPROPERTY()
	FName Name;

	/** Type of the context data, See EWorldConditionContextDataType. */
	UPROPERTY()
	EWorldConditionContextDataType Type = EWorldConditionContextDataType::Dynamic;
};

/**
 * Describes a reference to context data defined in UWorldConditionSchema.
 * When placed as editable property on a World Condition, allows the user to define which context data to use,
 * and allows quick access of that data via FWorldConditionContext.
 */
USTRUCT()
struct FWorldConditionContextDataRef
{
	GENERATED_BODY()

	/** Marker for invalid index value. */
	static constexpr uint8 InvalidIndex = 0xff;

	FWorldConditionContextDataRef() = default;

	explicit FWorldConditionContextDataRef(const FName InName, const uint8 InIndex = InvalidIndex)
		: Name(InName)
		, Index(InIndex)
	{
	}
	
	/** @return true if the data ref index is resolved. */
	bool IsValid() const { return Index != InvalidIndex; } 

	/** @return Referenced context data index. */
	uint8 GetIndex() const { return Index; }

	/** @return Name of the referenced data. */
	FName GetName() const { return Name; }
	
protected:
	UPROPERTY()
	FName Name;

	UPROPERTY()
	uint8 Index = InvalidIndex;

	friend class UWorldConditionSchema;
	friend class FWorldConditionContextDataRefDetails;
};

/**
 * View holding a pointer to context data struct or object.
 */
struct FWorldConditionDataView
{
	FWorldConditionDataView() = default;

	explicit FWorldConditionDataView(const EWorldConditionContextDataType InType)
		: Type(InType)
	{
	}

	// USTRUCT() constructor.
	explicit FWorldConditionDataView(const UScriptStruct* InScriptStruct, const uint8* InMemory, const EWorldConditionContextDataType InType)
		: Struct(InScriptStruct)
		, Memory(InMemory)
		, Type(InType)
	{
		// Must have type with valid pointer.
		check(!Memory || (Memory && Struct));
	}

	// UOBJECT() constructor.
	explicit FWorldConditionDataView(const UObject* Object, const EWorldConditionContextDataType InType)
		: Struct(Object ? Object->GetClass() : nullptr)
		, Memory(reinterpret_cast<const uint8*>(Object))
		, Type(InType)
	{
		// Must have type with valid pointer.
		check(!Memory || (Memory && Struct));
	}

	// USTRUCT() from a StructView.
	explicit FWorldConditionDataView(FConstStructView StructView, const EWorldConditionContextDataType InType)
		: Struct(StructView.GetScriptStruct())
		, Memory(StructView.GetMemory())
		, Type(InType)
	{
		// Must have type with valid pointer.
		check(!Memory || (Memory && Struct));
	}

	/** @return True if the view is set to valid data. */
	bool IsValid() const
	{
		return Memory != nullptr && Struct != nullptr;
	}

	/** @return Pointer to object stored in the view. */
	template <typename T>
	typename TEnableIf<TIsDerivedFrom<T, UObject>::IsDerived, T*>::Type GetPtr() const
	{
		// If Memory is set, expect Struct too. Otherwise, let nulls pass through.
		check(!Memory || (Memory && Struct));
		check(!Struct || Struct->IsChildOf(T::StaticClass()));
		return ((T*)Memory);
	}

	/** @return Pointer to struct stored in the view. */
	template <typename T>
    typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived, T*>::Type GetPtr() const
	{
		// If Memory is set, expect Struct too. Otherwise, let nulls pass through.
		check(!Memory || (Memory && Struct));
		check(!Struct || Struct->IsChildOf(TBaseStructure<T>::Get()));
		return ((T*)Memory);
	}

	/** @return Pointer to object stored in the view if the type matches correctly, else returns nullptr. */
	template <typename T>
	typename TEnableIf<TIsDerivedFrom<T, UObject>::IsDerived, T*>::Type TryGetPtr() const
	{
		// If Memory is set, expect Struct too. Otherwise, let nulls pass through.
		check(!Memory || (Memory && Struct));
		if (Struct && Struct->IsChildOf(T::StaticClass()))
		{
			return ((T*)Memory);
		}

		return nullptr;
	}

	/** @return Pointer to struct stored in the view if the type matches correctly, else returns nullptr. */
	template <typename T>
	typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived, T*>::Type TryGetPtr() const
	{
		// If Memory is set, expect Struct too. Otherwise, let nulls pass through.
		check(!Memory || (Memory && Struct));
		if (Struct && Struct->IsChildOf(TBaseStructure<T>::Get()))
		{
			return ((T*)Memory);
		}

		return nullptr;
	}

	/** @return Struct describing the data type. */
	const UStruct* GetStruct() const { return Struct; }

	/** @return Raw const pointer to the data. */
	const uint8* GetMemory() const { return Memory; }

	/** @return type of the context data. */
	EWorldConditionContextDataType GetType() const { return Type; }
	
protected:
	/** UClass or UScriptStruct of the data. */
	const UStruct* Struct = nullptr;

	/** Memory pointing at the class or struct */
	const uint8* Memory = nullptr;

	/** Type of the context data, copied from the descriptor. */
	EWorldConditionContextDataType Type = EWorldConditionContextDataType::Dynamic;
};
