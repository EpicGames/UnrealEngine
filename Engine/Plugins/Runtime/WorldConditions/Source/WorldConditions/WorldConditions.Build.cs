// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WorldConditions : ModuleRules
	{
		public WorldConditions(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new[] {
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new[] {
					"PropertyBindingUtils",
				}
			);
		}
	}
}
