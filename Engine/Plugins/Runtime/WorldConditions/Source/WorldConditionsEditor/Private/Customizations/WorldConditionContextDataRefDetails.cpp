// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionContextDataRefDetails.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SComboButton.h"
#include "DetailLayoutBuilder.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "WorldConditionQuery.h"
#include "WorldConditionSchema.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "WorldCondition"

TSharedRef<IPropertyTypeCustomization> FWorldConditionContextDataRefDetails::MakeInstance()
{
	return MakeShareable(new FWorldConditionContextDataRefDetails);
}

void FWorldConditionContextDataRefDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	CacheContextData();
	
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FWorldConditionContextDataRefDetails::OnGetContent)
			.ContentPadding(FMargin(6.f, 0.f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FWorldConditionContextDataRefDetails::GetCurrentDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

void FWorldConditionContextDataRefDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FWorldConditionContextDataRefDetails::CacheContextData()
{
	// Base struct
	static const FName BaseStructMetaName(TEXT("BaseStruct")); // TODO: move these names into one central place.
	static const FName BaseClassMetaName(TEXT("BaseClass")); // TODO: move these names into one central place.

	const FString BaseStructsName = StructProperty->GetMetaData(BaseStructMetaName);
	const FString BaseClassesName = StructProperty->GetMetaData(BaseClassMetaName);

	TArray<FString> ParsedStructsNames;
	TArray<FString> ParsedClassesNames;
	
	if (!BaseStructsName.IsEmpty())
	{
		BaseStructsName.ParseIntoArray(ParsedStructsNames, TEXT(","), true);
	}

	if (!BaseClassesName.IsEmpty())
	{
		BaseClassesName.ParseIntoArray(ParsedClassesNames, TEXT(","), true);
	}

	for (FString& StructNameString : ParsedStructsNames)
	{
		StructNameString.TrimStartAndEndInline();
		if (UStruct* StructType = UClass::TryFindTypeSlow<UScriptStruct>(StructNameString))
		{
			BaseStructs.Emplace(StructType);
		}
	}

	for (FString& ClassNameString : ParsedClassesNames)
	{
		ClassNameString.TrimStartAndEndInline();
		if (UStruct* StructType = UClass::TryFindTypeSlow<UClass>(ClassNameString))
		{
			BaseStructs.Emplace(StructType);
		}
	}

	if (BaseStructs.IsEmpty())
	{
		check(StructProperty.IsValid() && StructProperty->IsValidHandle())
		UE_LOGF(LogWorldCondition, Error, "%ls: Could not find BaseStructs '%ls' nor BaseClasses '%ls' based on the property metadata, expecting full struct name separated by commas if multiple.",
			*FString(StructProperty->GetPropertyPath()), *BaseStructsName, *BaseClassesName);
	}

	// Find schema from outer FWorldConditionQueryDefinition.
	Schema = nullptr;
	
	TSharedPtr<IPropertyHandle> CurrentProperty = StructProperty;
	while (CurrentProperty.IsValid() && !Schema)
	{
		const FProperty* Property = CurrentProperty->GetProperty();
		if (const FStructProperty* CurrentStructProperty = CastField<FStructProperty>(Property))
		{
			if (CurrentStructProperty->Struct == TBaseStructure<FWorldConditionQueryDefinition>::Get())
			{
				// Get schema from definition
				TArray<void*> RawNodeData;
				CurrentProperty->AccessRawData(RawNodeData);
				for (void* Data : RawNodeData)
				{
					if (const FWorldConditionQueryDefinition* QueryDefinition = static_cast<FWorldConditionQueryDefinition*>(Data))
					{
						Schema = QueryDefinition->GetSchemaClass().GetDefaultObject();
						if (Schema)
						{
							break;
						}
					}
				}
			}
		}

		CurrentProperty = CurrentProperty->GetParentHandle();
	}

	if (Schema && !BaseStructs.IsEmpty())
	{
		CachedContextData.Reset();
		for (const FWorldConditionContextDataDesc& Desc : Schema->GetContextDataDescs())
		{
			for (const UStruct* BaseStruct : BaseStructs)
			{
				if (Desc.MatchesType(BaseStruct))
				{
					CachedContextData.AddUnique(Desc.Name);
				}
			}
		}
	}
}

TSharedRef<SWidget> FWorldConditionContextDataRefDetails::OnGetContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	FUIAction ClearItemAction(FExecuteAction::CreateSP(const_cast<FWorldConditionContextDataRefDetails*>(this), &FWorldConditionContextDataRefDetails::OnSetContextData, FName()));
	MenuBuilder.AddMenuEntry(LOCTEXT("None", "None"), FText::GetEmpty(), FSlateIcon(), ClearItemAction);

	if (CachedContextData.Num() > 0)
	{
		MenuBuilder.AddSeparator();
	}
	
	for (const FName& ContextDataName : CachedContextData)
	{
		FUIAction ItemAction(FExecuteAction::CreateSP(const_cast<FWorldConditionContextDataRefDetails*>(this), &FWorldConditionContextDataRefDetails::OnSetContextData, ContextDataName));
		MenuBuilder.AddMenuEntry(FText::FromName(ContextDataName), FText::GetEmpty(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

FText FWorldConditionContextDataRefDetails::GetCurrentDesc() const
{
	if (!Schema)
	{
		return FText::GetEmpty();
	}
	
	if (const FWorldConditionContextDataRef* DataRef = GetCommonContextDataRef())
	{
		if (const FWorldConditionContextDataDesc* Desc = Schema->GetContextDataDescByName(DataRef->Name, BaseStructs))
		{
			return FText::FromName(Desc->Name);
		}
		return LOCTEXT("None", "None");
	}
	
	return LOCTEXT("MultipleSelected", "Multiple Selected");
}

FWorldConditionContextDataRef* FWorldConditionContextDataRefDetails::GetCommonContextDataRef() const
{
	check(StructProperty);

	FWorldConditionContextDataRef* Result = nullptr;
	
	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);

	for (void* Data : RawNodeData)
	{
		if (Data)
		{
			Result = static_cast<FWorldConditionContextDataRef*>(Data);
			break;
		}
	}

	return Result;
}

void FWorldConditionContextDataRefDetails::OnSetContextData(const FName ContextDataName) const
{
	check(StructProperty);

	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);

	GEditor->BeginTransaction(LOCTEXT("SelectReference", "Select Reference"));

	StructProperty->NotifyPreChange();

	for (void* Data : RawNodeData)
	{
		if (FWorldConditionContextDataRef* DataRef = static_cast<FWorldConditionContextDataRef*>(Data))
		{
			DataRef->Name = ContextDataName;
			DataRef->Index = FWorldConditionContextDataRef::InvalidIndex;
		}
	}

	StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructProperty->NotifyFinishedChangingProperties();

	GEditor->EndTransaction();
}

#undef LOCTEXT_NAMESPACE
