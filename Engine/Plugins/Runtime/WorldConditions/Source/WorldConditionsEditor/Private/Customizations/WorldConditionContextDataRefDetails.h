// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class IPropertyUtilities;
class SWidget;
struct FWorldConditionContextDataRef;
class UWorldConditionSchema;

/**
 * Type customization for FWorldConditionContextDataRef.
 */

class FWorldConditionContextDataRefDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:
	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	void CacheContextData();
	TSharedRef<SWidget> OnGetContent() const;
	FText GetCurrentDesc() const;

	void OnSetContextData(const FName ContextDataName) const;
	FWorldConditionContextDataRef* GetCommonContextDataRef() const;
	
	TArray<FName> CachedContextData;

	const UWorldConditionSchema* Schema = nullptr;
	TArray<const UStruct*> BaseStructs;
	
	IPropertyUtilities* PropUtils = nullptr;
	TSharedPtr<IPropertyHandle> StructProperty;
};
