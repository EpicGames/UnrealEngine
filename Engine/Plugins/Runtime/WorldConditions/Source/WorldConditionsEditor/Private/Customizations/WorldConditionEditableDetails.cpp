// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionEditableDetails.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SComboButton.h"
#include "DetailLayoutBuilder.h"
#include "WorldConditionQuery.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/StyleColors.h"
#include "StructViewerModule.h"
#include "StructViewerFilter.h"
#include "WorldConditionEditorStyle.h"
#include "InstancedStructDetails.h"
#include "WorldConditionSchema.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "WorldCondition"

class FWorldConditionStructFilter : public IStructViewerFilter
{
public:
	/** The base struct for the property that classes must be a child-of. */
	const UScriptStruct* BaseStruct = nullptr;
	const UWorldConditionSchema* Schema = nullptr;

	virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
	{
		if (!InStruct || InStruct == BaseStruct)
		{
			return false;
		}

		if (Schema && !Schema->IsStructAllowed(InStruct))
		{
			return false;
		}

		if (InStruct->HasMetaData(TEXT("Hidden")))
		{
			return false;
		}

		// Query the native struct to see if it has the correct parent type (if any)
		return !BaseStruct || InStruct->IsChildOf(BaseStruct);
	}

	virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
	{
		return false;
	}
};


TSharedRef<IPropertyTypeCustomization> FWorldConditionEditableDetails::MakeInstance()
{
	return MakeShareable(new FWorldConditionEditableDetails);
}

void FWorldConditionEditableDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	ConditionProperty = StructProperty->GetChildHandle(TEXT("Condition"));
	ExpressionDepthProperty = StructProperty->GetChildHandle(TEXT("ExpressionDepth"));
	OperatorProperty = StructProperty->GetChildHandle(TEXT("Operator"));
	InvertProperty = StructProperty->GetChildHandle(TEXT("bInvert"));

	check(ConditionProperty.IsValid());
	check(ExpressionDepthProperty.IsValid());
	check(OperatorProperty.IsValid());
	check(InvertProperty.IsValid());
	
	CacheSchema();

	const FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FWorldConditionEditableDetails::ShouldResetToDefault);
	const FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FWorldConditionEditableDetails::ResetToDefault);
	const FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);

	HeaderRow
		.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FWorldConditionEditableDetails::OnCopyCondition)))
		.PasteAction(FUIAction(
			FExecuteAction::CreateSP(this, &FWorldConditionEditableDetails::OnPasteCondition),
			FCanExecuteAction::CreateSP(this, &FWorldConditionEditableDetails::CanPasteCondition)))
		.WholeRowContent()
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(2.0f)
			[
				SNew(SHorizontalBox)
				// Indent
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(this, &FWorldConditionEditableDetails::GetIndentSize)
					[
						SNew(SComboButton)
						.ComboButtonStyle(FWorldConditionEditorStyle::Get(), "Condition.Indent.ComboBox")
						.ContentPadding(2.0f)
						.HasDownArrow(false)
						.VAlign(VAlign_Center)
						.OnGetMenuContent(this, &FWorldConditionEditableDetails::OnGetIndentContent)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text(FText::FromString(TEXT(">")))
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.ToolTipText(LOCTEXT("DepthTooltip", "Depth of the expression row, controls parentheses and evaluation order."))
						]
					]
				]
				// Operator
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(30.0f)
					.Padding(FMargin(2, 4))
					.VAlign(VAlign_Center)
					[
						SNew(SComboButton)
						.IsEnabled(TAttribute<bool>(this, &FWorldConditionEditableDetails::IsOperatorEnabled))
						.ComboButtonStyle(FWorldConditionEditorStyle::Get(), "Condition.Operator.ComboBox")
						.ButtonColorAndOpacity(this, &FWorldConditionEditableDetails::GetOperatorColor)
						.HasDownArrow(false)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnGetMenuContent(this, &FWorldConditionEditableDetails::OnGetOperatorContent)
						.ButtonContent()
						[
							SNew(STextBlock)
							.TextStyle(FWorldConditionEditorStyle::Get(), "Condition.Operator")
							.Text(this, &FWorldConditionEditableDetails::GetOperatorText)
						]
					]
				]
				// Open parens
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FWorldConditionEditorStyle::Get(), "Condition.Parens")
					.Text(this, &FWorldConditionEditableDetails::GetOpenParens)
				]

				// Not
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.Padding(FMargin(8, 0, 4, 0))
					.VAlign(VAlign_Center)
					.Visibility(this, &FWorldConditionEditableDetails::GetInvertVisibility)
					[
						SNew(STextBlock)
						.TextStyle(FWorldConditionEditorStyle::Get(), "Condition.Operator")
						.Text(LOCTEXT("NotOperator","NOT"))
					]
				]
				
				// Class picker
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(FMargin(4.0f, 0.0f, 0.0f, 0.0f)))
				.VAlign(VAlign_Center)
				[
					SAssignNew(ComboButton, SComboButton)
					.OnGetMenuContent(this, &FWorldConditionEditableDetails::GeneratePicker)
					.ContentPadding(0.f)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &FWorldConditionEditableDetails::GetDisplayValueString)
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]

				// Close parens
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.TextStyle(FWorldConditionEditorStyle::Get(), "Condition.Parens")
					.Text(this, &FWorldConditionEditableDetails::GetCloseParens)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				StructPropertyHandle->CreateDefaultPropertyButtonWidgets()
			]
		]
		.OverrideResetToDefault(ResetOverride);
}

void FWorldConditionEditableDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	check(ConditionProperty);
	check(InvertProperty);

	StructBuilder.AddProperty(InvertProperty.ToSharedRef());
	
	TSharedRef<FInstancedStructDataDetails> NodeDetails = MakeShareable(new FInstancedStructDataDetails(ConditionProperty));
	StructBuilder.AddCustomBuilder(NodeDetails);
}

void FWorldConditionEditableDetails::CacheSchema()
{
	// Find schema from outer FWorldConditionQueryDefinition.
	Schema = nullptr;
	
	TSharedPtr<IPropertyHandle> CurrentProperty = StructProperty;
	while (CurrentProperty.IsValid() && !Schema)
	{
		const FProperty* Property = CurrentProperty->GetProperty();
		if (const FStructProperty* CurrentStructProperty = CastField<FStructProperty>(Property))
		{
			if (CurrentStructProperty->Struct == TBaseStructure<FWorldConditionQueryDefinition>::Get())
			{
				// Get schema from definition
				TArray<void*> RawNodeData;
				CurrentProperty->AccessRawData(RawNodeData);
				for (void* Data : RawNodeData)
				{
					if (const FWorldConditionQueryDefinition* QueryDefinition = static_cast<FWorldConditionQueryDefinition*>(Data))
					{
						Schema = QueryDefinition->GetSchemaClass().GetDefaultObject();
						if (Schema)
						{
							break;
						}
					}
				}
			}
		}
		CurrentProperty = CurrentProperty->GetParentHandle(); 
	}
}

bool FWorldConditionEditableDetails::ShouldResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	check(StructProperty);
	
	bool bAnyValid = false;
	
	TArray<const void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);
	for (const void* Data : RawNodeData)
	{
		if (const FWorldConditionEditable* Condition = static_cast<const FWorldConditionEditable*>(Data))
		{
			if (Condition->Condition.IsValid())
			{
				bAnyValid = true;
				break;
			}
		}
	}
	
	// Assume that the default value is empty. Any valid means that some can be reset to empty.
	return bAnyValid;
}


void FWorldConditionEditableDetails::ResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	check(StructProperty);
	
	GEditor->BeginTransaction(LOCTEXT("OnResetToDefault", "Reset to default"));

	StructProperty->NotifyPreChange();
	
	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);
	for (void* Data : RawNodeData)
	{
		if (FWorldConditionEditable* Node = static_cast<FWorldConditionEditable*>(Data))
		{
			Node->Reset();
		}
	}

	StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructProperty->NotifyFinishedChangingProperties();

	GEditor->EndTransaction();

	if (PropUtils)
	{
		PropUtils->ForceRefresh();
	}
}

EVisibility FWorldConditionEditableDetails::GetInvertVisibility() const
{
	check(InvertProperty);
	
	bool bInvert = false;
	InvertProperty->GetValue(bInvert);
	
	return bInvert ? EVisibility::Visible : EVisibility::Collapsed;
}

FOptionalSize FWorldConditionEditableDetails::GetIndentSize() const
{
	return FOptionalSize(15.0f + GetExpressionDepth() * 30.0f);
}

TSharedRef<SWidget> FWorldConditionEditableDetails::OnGetIndentContent() const
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 Depth = 0; Depth < UE::WorldCondition::MaxExpressionDepth; Depth++)
	{
		FUIAction ItemAction(
			FExecuteAction::CreateSP(this, &FWorldConditionEditableDetails::SetExpressionDepth, Depth),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FWorldConditionEditableDetails::IsExpressionDepth, Depth));
		MenuBuilder.AddMenuEntry(FText::AsNumber(Depth), TAttribute<FText>(), FSlateIcon(), ItemAction, FName(), EUserInterfaceActionType::Check);
	}

	return MenuBuilder.MakeWidget();
}

int32 FWorldConditionEditableDetails::GetExpressionDepth() const
{
	check(ExpressionDepthProperty);
	
	uint8 Depth = 0;
	ExpressionDepthProperty->GetValue(Depth);

	return Depth;
}

void FWorldConditionEditableDetails::SetExpressionDepth(const int32 Depth) const
{
	check(ExpressionDepthProperty);
	
	ExpressionDepthProperty->SetValue((uint8)FMath::Clamp(Depth, 0, UE::WorldCondition::MaxExpressionDepth - 1));
}

bool FWorldConditionEditableDetails::IsExpressionDepth(const int32 Depth) const
{
	return Depth == GetExpressionDepth();
}

FText FWorldConditionEditableDetails::GetOperatorText() const
{
	check(OperatorProperty);

	// First item does not relate to anything existing, it could be empty, we return IF to indicate that we're building condition. 
	if (IsFirstItem())
	{
		return LOCTEXT("IfOperator", "IF");
	}

	uint8 Value = 0;
	OperatorProperty->GetValue(Value);
	const EWorldConditionOperator Operator = (EWorldConditionOperator)Value;

	if (Operator == EWorldConditionOperator::And)
	{
		return LOCTEXT("AndOperator", "AND");
	}
	else if (Operator == EWorldConditionOperator::Or)
	{
		return LOCTEXT("OrOperator", "OR");
	}
	else
	{
		ensureMsgf(false, TEXT("Unhandled Operator %s"), *UEnum::GetValueAsString(Operator));
	}

	return FText::GetEmpty();
}

FSlateColor FWorldConditionEditableDetails::GetOperatorColor() const
{
	check(OperatorProperty);

	if (IsFirstItem())
	{
		return FStyleColors::Transparent;
	}

	uint8 Value = 0; 
	OperatorProperty->GetValue(Value);
	const EWorldConditionOperator Operator = (EWorldConditionOperator)Value;

	if (Operator == EWorldConditionOperator::And)
	{
		return FStyleColors::AccentPink;
	}
	else if (Operator == EWorldConditionOperator::Or)
	{
		return FStyleColors::AccentBlue;
	}
	else
	{
		ensureMsgf(false, TEXT("Unhandled Operator %s"), *UEnum::GetValueAsString(Operator));
	}

	return FStyleColors::Transparent;

}

TSharedRef<SWidget> FWorldConditionEditableDetails::OnGetOperatorContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	FUIAction AndAction(
		FExecuteAction::CreateSP(this, &FWorldConditionEditableDetails::SetOperator, EWorldConditionOperator::And),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FWorldConditionEditableDetails::IsOperator, EWorldConditionOperator::And));
	MenuBuilder.AddMenuEntry(LOCTEXT("AndOperator", "AND"), TAttribute<FText>(), FSlateIcon(), AndAction, FName(), EUserInterfaceActionType::Check);

	FUIAction OrAction(FExecuteAction::CreateSP(this, &FWorldConditionEditableDetails::SetOperator, EWorldConditionOperator::Or),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FWorldConditionEditableDetails::IsOperator, EWorldConditionOperator::Or));
	MenuBuilder.AddMenuEntry(LOCTEXT("OrOperator", "OR"), TAttribute<FText>(), FSlateIcon(), OrAction, FName(), EUserInterfaceActionType::Check);

	return MenuBuilder.MakeWidget();
}

bool FWorldConditionEditableDetails::IsOperatorEnabled() const
{
	return !IsFirstItem();
}

void FWorldConditionEditableDetails::SetOperator(const EWorldConditionOperator Operator) const
{
	check(OperatorProperty);
	
	OperatorProperty->SetValue((uint8)Operator);
}

bool FWorldConditionEditableDetails::IsOperator(const EWorldConditionOperator Operator) const
{
	check(OperatorProperty);

	uint8 Value = 0; 
	OperatorProperty->GetValue(Value);
	const EWorldConditionOperator CurrOperator = (EWorldConditionOperator)Value;

	return CurrOperator == Operator;
}

bool FWorldConditionEditableDetails::IsFirstItem() const
{
	check(StructProperty);
	return StructProperty->GetIndexInArray() == 0;
}

int32 FWorldConditionEditableDetails::GetCurrExpressionDepth() const
{
	// First item needs to be zero depth to make the parentheses counting to work properly.
	return IsFirstItem() ? 0 : GetExpressionDepth();
}

int32 FWorldConditionEditableDetails::GetNextExpressionDepth() const
{
	// Find the intent of the next item by finding the item in the parent array.
	check(StructProperty);
	TSharedPtr<IPropertyHandle> ParentProp = StructProperty->GetParentHandle();
	if (!ParentProp.IsValid())
	{
		return 0;
	}
	TSharedPtr<IPropertyHandleArray> ParentArray = ParentProp->AsArray();
	if (!ParentArray.IsValid())
	{
		return 0;
	}

	uint32 NumElements = 0;
	if (ParentArray->GetNumElements(NumElements) != FPropertyAccess::Success)
	{
		return 0;
	}
	
	const int32 NextIndex = StructProperty->GetIndexInArray() + 1;
	if (NextIndex >= (int32)NumElements)
	{
		return 0;
	}

	TSharedPtr<IPropertyHandle> NextStructProperty = ParentArray->GetElement(NextIndex);
	if (!NextStructProperty.IsValid())
	{
		return 0;
	}
	
	TSharedPtr<IPropertyHandle> NextExpressionDepthProperty = NextStructProperty->GetChildHandle(TEXT("ExpressionDepth"));
	if (!NextExpressionDepthProperty.IsValid())
	{
		return 0;
	}
	
	uint8 Indent = 0;
	NextExpressionDepthProperty->GetValue(Indent);

	return Indent;
}
	
FText FWorldConditionEditableDetails::GetOpenParens() const
{
	check(ExpressionDepthProperty);

	const int32 CurrDepth = GetCurrExpressionDepth();
	const int32 NextDepth = GetNextExpressionDepth();
	const int32 DeltaDepth = NextDepth - CurrDepth;
	const int32 OpenParens = FMath::Max(0, DeltaDepth);

	FText Result = FText::GetEmpty();
	
	static_assert(UE::WorldCondition::MaxExpressionDepth == 4);
	switch (OpenParens)
	{
	case 0:
		break;
	case 1:
		Result = FText::FromString(TEXT("("));
		break;
	case 2:
		Result = FText::FromString(TEXT("(("));
		break;
	case 3:
		Result = FText::FromString(TEXT("((("));
		break;
	case 4:
		Result = FText::FromString(TEXT("(((("));
		break;
	default:
		ensureMsgf(false, TEXT("Max %d parenthesis supported, got %d."), UE::WorldCondition::MaxExpressionDepth, OpenParens);
	}

	return Result;
}

FText FWorldConditionEditableDetails::GetCloseParens() const
{
	check(ExpressionDepthProperty);

	const int32 CurrDepth = GetCurrExpressionDepth();
	const int32 NextDepth = GetNextExpressionDepth();
	const int32 DeltaDepth = NextDepth - CurrDepth;
	const int32 CloseParens = FMath::Max(0, -DeltaDepth);

	FText Result = FText::GetEmpty();
	
	static_assert(UE::WorldCondition::MaxExpressionDepth == 4);
	switch (CloseParens)
	{
	case 0:
		break;
	case 1:
		Result = FText::FromString(TEXT(")"));
		break;
	case 2:
		Result = FText::FromString(TEXT("))"));
		break;
	case 3:
		Result = FText::FromString(TEXT(")))"));
		break;
	case 4:
		Result = FText::FromString(TEXT("))))"));
		break;
	default:
		ensureMsgf(false, TEXT("Max %d parenthesis supported, got %d."), UE::WorldCondition::MaxExpressionDepth, CloseParens);
	}

	return Result;
}

FText FWorldConditionEditableDetails::GetDisplayValueString() const
{
	if (const FWorldConditionEditable* EditableCondition = GetCommonCondition())
	{
		if (const FWorldConditionBase* Condition = EditableCondition->Condition.GetPtr<FWorldConditionBase>())
		{
			return Condition->GetDescription();
		}
	}
	return LOCTEXT("ValueNone", "None");
}

TSharedRef<SWidget> FWorldConditionEditableDetails::GeneratePicker()
{
	TSharedRef<FWorldConditionStructFilter> StructFilter = MakeShared<FWorldConditionStructFilter>();
	StructFilter->BaseStruct = TBaseStructure<FWorldConditionBase>::Get();
	StructFilter->Schema = Schema;

	FStructViewerInitializationOptions Options;
	Options.bShowNoneOption = true;
	Options.StructFilter = StructFilter;
	Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
	Options.DisplayMode = EStructViewerDisplayMode::ListView;
	Options.bAllowViewOptions = true;

	const FOnStructPicked OnPicked(FOnStructPicked::CreateRaw(this, &FWorldConditionEditableDetails::OnStructPicked));

	return SNew(SBox)
		.WidthOverride(280)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(500)
			[
				FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer").CreateStructViewer(Options, OnPicked)
			]
		];
}

void FWorldConditionEditableDetails::OnStructPicked(const UScriptStruct* InStruct) const
{
	check(StructProperty);

	TArray<void*> RawNodeData;
	StructProperty->AccessRawData(RawNodeData);

	GEditor->BeginTransaction(LOCTEXT("SelectCondition", "Select Condition"));

	StructProperty->NotifyPreChange();

	for (void* Data : RawNodeData)
	{
		if (FWorldConditionEditable* Condition = static_cast<FWorldConditionEditable*>(Data))
		{
			Condition->Condition.InitializeAs(InStruct);
		}
	}

	StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructProperty->NotifyFinishedChangingProperties();

	GEditor->EndTransaction();

	ComboButton->SetIsOpen(false);

	if (PropUtils)
	{
		PropUtils->ForceRefresh();
	}
}

void FWorldConditionEditableDetails::OnCopyCondition() const
{
	FString Value;
	StructProperty->GetValueAsFormattedString(Value, PPF_Copy);
	FPlatformApplicationMisc::ClipboardCopy(*Value);
}

bool FWorldConditionEditableDetails::CanPasteCondition() const
{
	// Never allowing pasting with no schema, we are not supporting anything basically.
	if (!Schema)
	{
		return false;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (ClipboardText.IsEmpty())
	{
		return false;
	}

	// Try to deserialize the clipboard data as a FWorldConditionEditable.
	FWorldConditionEditable TempEditable;
	const TCHAR* Buffer = *ClipboardText;
	const TCHAR* Result = FWorldConditionEditable::StaticStruct()->ImportText(
		Buffer, &TempEditable, nullptr, PPF_Copy, nullptr, FWorldConditionEditable::StaticStruct()->GetName());
	if (!Result)
	{
		return false;
	}

	// If the condition is valid, check if the struct type is allowed.
	if (TempEditable.Condition.IsValid())
	{
		return Schema->IsStructAllowed(TempEditable.Condition.GetScriptStruct());
	}

	return true;
}

void FWorldConditionEditableDetails::OnPasteCondition() const
{
	FString Value;
	FPlatformApplicationMisc::ClipboardPaste(Value);

	if (!Value.IsEmpty())
	{
		FScopedTransaction Transaction(LOCTEXT("PasteCondition", "Paste Condition"));
		StructProperty->SetValueFromFormattedString(Value, EPropertyValueSetFlags::InstanceObjects);
	}
}

const FWorldConditionEditable* FWorldConditionEditableDetails::GetCommonCondition() const
{
	check(StructProperty);

	bool bMultipleValues = false;
	TArray<void*> RawData;
	StructProperty->AccessRawData(RawData);

	const FWorldConditionEditable* CommonCondition = nullptr;
	
	for (void* Data : RawData)
	{
		if (const FWorldConditionEditable* Condition = static_cast<FWorldConditionEditable*>(Data))
		{
			if (!bMultipleValues && Condition)
			{
				CommonCondition = Condition;
			}
			else if (CommonCondition != Condition)
			{
				CommonCondition = nullptr;
				bMultipleValues = true;
			}
		}
	}

	return CommonCondition;
}


#undef LOCTEXT_NAMESPACE
