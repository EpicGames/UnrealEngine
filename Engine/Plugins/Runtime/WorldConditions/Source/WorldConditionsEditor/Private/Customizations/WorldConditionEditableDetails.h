// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

struct EVisibility;
struct FOptionalSize;

class IPropertyHandle;
class IPropertyUtilities;
class SWidget;

enum class EWorldConditionOperator : uint8;
struct FWorldConditionEditable;
class UWorldConditionSchema;

/**
 * Type customization for FWorldConditionEditable.
 */
class FWorldConditionEditableDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:
	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	void CacheSchema();

	bool ShouldResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	void ResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);

	EVisibility GetInvertVisibility() const;

	FOptionalSize GetIndentSize() const;
	TSharedRef<SWidget> OnGetIndentContent() const;

	int32 GetExpressionDepth() const;
	void SetExpressionDepth(const int32 Depth) const;
	bool IsExpressionDepth(const int32 Depth) const;

	FText GetOperatorText() const;
	FSlateColor GetOperatorColor() const;
	TSharedRef<SWidget> OnGetOperatorContent() const;
	bool IsOperatorEnabled() const;

	void SetOperator(const EWorldConditionOperator Operator) const;
	bool IsOperator(const EWorldConditionOperator Operator) const;

	bool IsFirstItem() const;
	int32 GetCurrExpressionDepth() const;
	int32 GetNextExpressionDepth() const;
	
	FText GetOpenParens() const;
	FText GetCloseParens() const;

	FText GetDisplayValueString() const;

	TSharedRef<SWidget> GeneratePicker();
	void OnStructPicked(const UScriptStruct* InStruct) const;

	void OnCopyCondition() const;
	bool CanPasteCondition() const;
	void OnPasteCondition() const;
	
	const FWorldConditionEditable* GetCommonCondition() const;

	const UWorldConditionSchema* Schema = nullptr;
	
	TSharedPtr<class SComboButton> ComboButton;
	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyHandle> ConditionProperty;
	TSharedPtr<IPropertyHandle> ExpressionDepthProperty;
	TSharedPtr<IPropertyHandle> OperatorProperty;
	TSharedPtr<IPropertyHandle> InvertProperty;
	IPropertyUtilities* PropUtils = nullptr;
};
