// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionQueryDefinitionDetails.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "WorldConditionQuery.h"

#define LOCTEXT_NAMESPACE "WorldCondition"

TSharedRef<IPropertyTypeCustomization> FWorldConditionQueryDefinitionDetails::MakeInstance()
{
	return MakeShareable(new FWorldConditionQueryDefinitionDetails);
}

void FWorldConditionQueryDefinitionDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	EditableConditionsProperty = StructProperty->GetChildHandle(TEXT("EditableConditions"));

	// Keep the definition up to date as it's being edited.
	StructProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FWorldConditionQueryDefinitionDetails::InitializeDefinition));
	StructProperty->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FWorldConditionQueryDefinitionDetails::InitializeDefinition));
	
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(170.f)
		[
			EditableConditionsProperty->CreatePropertyValueWidget()
		];
}

void FWorldConditionQueryDefinitionDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	check(EditableConditionsProperty.IsValid());

	// Place editable conditions directly under.
	if (EditableConditionsProperty.IsValid())
	{
		const TSharedRef<FDetailArrayBuilder> Builder = MakeShareable(new FDetailArrayBuilder(EditableConditionsProperty.ToSharedRef(), /*InGenerateHeader*/ false, /*InDisplayResetToDefault*/ true, /*InDisplayElementNum*/ false));
		Builder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda([](TSharedRef<IPropertyHandle> PropertyHandle, int32 ArrayIndex, IDetailChildrenBuilder& ChildrenBuilder)
		{
			ChildrenBuilder.AddProperty(PropertyHandle);
		}));
		StructBuilder.AddCustomBuilder(Builder);
	}
}

void FWorldConditionQueryDefinitionDetails::InitializeDefinition() const
{
	check(StructProperty);

	TArray<void*> RawData;
	TArray<UObject*> OuterObjects;
		
	StructProperty->AccessRawData(RawData);
	StructProperty->GetOuterObjects(OuterObjects);

	// If no outer objects, try to find an outer asset (can happen e.g. in data table).
	if (OuterObjects.Num() == 0)
	{
		TArray<UPackage*> OuterPackages;
		StructProperty->GetOuterPackages(OuterPackages);
		for (UPackage* Package : OuterPackages)
		{
			if (UObject* Asset = Package->FindAssetInPackage())
			{
				OuterObjects.Add(Asset);
			}
		}
		// Number of objects from the packages should match number of data.
		if (OuterObjects.Num() != RawData.Num())
		{
			ensureMsgf(false, TEXT("Cannot edit World Condition Definition in this context, cannot find a valid outer object."));
			return;
		}
	}

	check(RawData.Num() == OuterObjects.Num());
	
	for (int32 Index = 0; Index < RawData.Num(); Index++)
	{
		if (FWorldConditionQueryDefinition* QueryDefinition = static_cast<FWorldConditionQueryDefinition*>(RawData[Index]))
		{
			QueryDefinition->Initialize(OuterObjects[Index]);
		}
	}
}


#undef LOCTEXT_NAMESPACE
