// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class IPropertyUtilities;
class SWidget;

/**
 * Type customization for FWorldConditionQueryDefinition.
 */
class FWorldConditionQueryDefinitionDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:
	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	void InitializeDefinition() const;
	
	IPropertyUtilities* PropUtils = nullptr;
	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyHandle> EditableConditionsProperty;
};
