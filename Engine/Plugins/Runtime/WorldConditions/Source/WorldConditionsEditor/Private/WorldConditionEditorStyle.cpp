// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionEditorStyle.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"
#include "Styling/CoreStyle.h"

TUniquePtr<FWorldConditionEditorStyle> FWorldConditionEditorStyle::Instance(nullptr);
FColor FWorldConditionEditorStyle::TypeColor(104,49,178);

FWorldConditionEditorStyle::FWorldConditionEditorStyle() : FSlateStyleSet("WorldConditionEditorStyle")
{
	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FComboButtonStyle& ComboButtonStyle = FCoreStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton");

	// Condition Operator combo button
	const FButtonStyle OperatorButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentGreen.GetSpecifiedColor().Desaturate(0.3f), 4.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::AccentGreen.GetSpecifiedColor().Desaturate(0.2f), 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::AccentGreen.GetSpecifiedColor().Desaturate(0.1f), 4.0f))
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::ForegroundHover)
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1));

	Set("Condition.Operator.ComboBox", FComboButtonStyle(ComboButtonStyle).SetButtonStyle(OperatorButton));

	Set("Condition.Operator", FTextBlockStyle(NormalText)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
		.SetFontSize(7));

	Set("Condition.Parens", FTextBlockStyle(NormalText)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.SetFontSize(12));

	// Condition Indent combo button
	const FButtonStyle IndentButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FLinearColor::Transparent, 2.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Background, 2.0f, FStyleColors::InputOutline, 1.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Background, 2.0f, FStyleColors::Hover, 1.0f))
		.SetNormalForeground(FStyleColors::Transparent)
		.SetHoveredForeground(FStyleColors::Hover)
		.SetPressedForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1));
	
	Set("Condition.Indent.ComboBox", FComboButtonStyle(ComboButtonStyle).SetButtonStyle(IndentButton));

	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FWorldConditionEditorStyle::~FWorldConditionEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FWorldConditionEditorStyle& FWorldConditionEditorStyle::Get()
{
	if (!Instance.IsValid())
	{
		Instance = TUniquePtr<FWorldConditionEditorStyle>(new FWorldConditionEditorStyle);
	}
	return *(Instance.Get());
}

void FWorldConditionEditorStyle::Shutdown()
{
	Instance.Reset();
}
