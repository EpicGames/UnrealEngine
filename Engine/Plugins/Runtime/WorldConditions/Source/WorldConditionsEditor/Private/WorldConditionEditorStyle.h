// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FWorldConditionEditorStyle final : public FSlateStyleSet
{
public:
	virtual ~FWorldConditionEditorStyle() override;

	static FWorldConditionEditorStyle& Get();
	static void Shutdown();

	static FColor TypeColor;
private:
	FWorldConditionEditorStyle();

	static TUniquePtr<FWorldConditionEditorStyle> Instance;
};
