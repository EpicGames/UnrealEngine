// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionsEditorModule.h"

#include "ComponentVisualizer.h" // IWYU pragma: keep
#include "PropertyEditorModule.h"
#include "WorldConditionEditorStyle.h"
#include "Customizations/WorldConditionEditableDetails.h"
#include "Customizations/WorldConditionQueryDefinitionDetails.h"
#include "Customizations/WorldConditionContextDataRefDetails.h"

#define LOCTEXT_NAMESPACE "WorldConditions"

class FWorldConditionsEditorModule : public IWorldConditionsEditorModule
{
protected:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer);

private:
	TArray<FName> RegisteredComponentClassNames;
};

void FWorldConditionsEditorModule::StartupModule()
{
	FWorldConditionEditorStyle::Get();
	
	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("WorldConditionEditable", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWorldConditionEditableDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("WorldConditionQueryDefinition", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWorldConditionQueryDefinitionDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout("WorldConditionContextDataRef", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWorldConditionContextDataRefDetails::MakeInstance));

	PropertyModule.NotifyCustomizationModuleChanged();
}

void FWorldConditionsEditorModule::ShutdownModule()
{
	// Unregister the details customization
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout("WorldConditionEditable");
		PropertyModule.UnregisterCustomPropertyTypeLayout("WorldConditionQueryDefinition");
		PropertyModule.UnregisterCustomPropertyTypeLayout("WorldConditionContextDataRef");
	}
	
	FWorldConditionEditorStyle::Shutdown();
}

IMPLEMENT_MODULE(FWorldConditionsEditorModule, WorldConditionsEditor)

#undef LOCTEXT_NAMESPACE
