// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WorldConditionsEditor : ModuleRules
	{
		public WorldConditionsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new [] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"PropertyEditor",
					"SlateCore",
					"Slate",
					"WorldConditions",
					"UnrealEd",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new [] {
					"RenderCore",
					"ApplicationCore",
					"StructUtilsEditor",
				}
			);
		}

	}
}
