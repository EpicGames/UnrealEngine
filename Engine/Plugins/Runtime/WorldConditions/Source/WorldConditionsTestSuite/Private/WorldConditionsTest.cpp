// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "Engine/World.h"
#include "WorldConditionTestTypes.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "UObject/UObjectThreadContext.h"

#define LOCTEXT_NAMESPACE "AITestSuite_WorldConditionsTest"

UE_DISABLE_OPTIMIZATION_SHIP

struct FWorldConditionTest_Init : FAITestBase
{
	virtual bool InstantTest() override
	{
		FWorldConditionQueryDefinition Definition;
		const bool bInitialized = Definition.Initialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(),
			{
				FWorldConditionEditable(0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTest())),
				FWorldConditionEditable(0, EWorldConditionOperator::And, FConstStructView::Make(FWorldConditionTest())),
			});

		AITEST_TRUE("Query definition should get initialized", bInitialized);

		FWorldConditionQueryState State;
		State.Initialize(GetWorld(), Definition);
		AITEST_TRUE("Query state should get initialized", State.IsInitialized());
		AITEST_EQUAL("Query state should have 2 conditions", State.GetNumConditions(), 2);
		AITEST_EQUAL("Query state condition 1 operator should be And", State.GetItem(1).Operator, EWorldConditionOperator::And);
		
		State.Free();
		AITEST_EQUAL("Query state should have 0 conditions", State.GetNumConditions(), 0);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_Init, "System.AI.WorldConditions.Init");

struct FWorldConditionTest_Eval : FAITestBase
{
	virtual bool InstantTest() override
	{
		FWorldConditionQuery Query;
		const bool bInitialized = Query.DebugInitialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(),
			{
				FWorldConditionEditable(0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTest(1))),
				FWorldConditionEditable(0, EWorldConditionOperator::And, FConstStructView::Make(FWorldConditionTest(1)))
			});

		AITEST_TRUE("Query should get initialized", bInitialized);
		
		const FWorldConditionTestData TestData(1);

		const UWorldConditionTestSchema* TestSchema = GetDefault<UWorldConditionTestSchema>();
		FWorldConditionContextData ContextData(*TestSchema);
		const bool bContextDataSet = ContextData.SetContextData(TestSchema->GetValueRef(), &TestData);
		AITEST_TRUE("Query context data should be set", bContextDataSet);
		
		const bool bActivated = Query.Activate(GetWorld(), ContextData);
		AITEST_TRUE("Query should get activated", bActivated);
		AITEST_TRUE("Query should be active", Query.IsActive());
		
		const bool bResult = Query.IsTrue(ContextData);
		AITEST_TRUE("Query result should be true", bResult);

		Query.Deactivate(ContextData);
		AITEST_FALSE("Query should not be active", Query.IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_Eval, "System.AI.WorldConditions.Eval");

struct FWorldConditionTest_EvalInvert : FAITestBase
{
	virtual bool InstantTest() override
	{
		FWorldConditionQuery Query;
		const bool bInitialized = Query.DebugInitialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(),
			{
				FWorldConditionEditable(0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTest(1))),
				FWorldConditionEditable(0, EWorldConditionOperator::And, /*bInvert*/true, FConstStructView::Make(FWorldConditionTest(0)))
			});

		AITEST_TRUE("Query should get initialized", bInitialized);
		
		const FWorldConditionTestData TestData(1);

		const UWorldConditionTestSchema* TestSchema = GetDefault<UWorldConditionTestSchema>();
		FWorldConditionContextData ContextData(*TestSchema);
		const bool bContextDataSet = ContextData.SetContextData(TestSchema->GetValueRef(), &TestData);
		AITEST_TRUE("Query context data should be set", bContextDataSet);
		
		const bool bActivated = Query.Activate(GetWorld(), ContextData);
		AITEST_TRUE("Query should get activated", bActivated);
		AITEST_TRUE("Query should be active", Query.IsActive());
		
		const bool bResult = Query.IsTrue(ContextData);
		AITEST_TRUE("Query result should be true", bResult);

		Query.Deactivate(ContextData);
		AITEST_FALSE("Query should not be active", Query.IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_EvalInvert, "System.AI.WorldConditions.EvalInvert");

struct FWorldConditionTest_CachedEval : FAITestBase
{
	virtual bool InstantTest() override
	{
		FWorldConditionQuery Query;
		const bool bInitialized = Query.DebugInitialize(&GetWorld(), UWorldConditionTestCachedSchema::StaticClass(),
			{
				FWorldConditionEditable(0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTestCached(1))),
				FWorldConditionEditable(0, EWorldConditionOperator::And, FConstStructView::Make(FWorldConditionTestCached(1)))
			});

		AITEST_TRUE("Query should get initialized", bInitialized);
		
		FWorldConditionTestData TestData(1);

		const UWorldConditionTestCachedSchema* TestSchema = GetDefault<UWorldConditionTestCachedSchema>();
		FWorldConditionContextData ContextData(*TestSchema);
		const bool bContextDataSet = ContextData.SetContextData(TestSchema->GetValueRef(), &TestData);
		AITEST_TRUE("Query context data should be set", bContextDataSet);
		
		const bool bActivated = Query.Activate(GetWorld(), ContextData);
		AITEST_TRUE("Query should get activated", bActivated);
		AITEST_TRUE("Query should be active", Query.IsActive());
		
		const bool bResult1 = Query.IsTrue(ContextData);
		AITEST_TRUE("Query result should be true", bResult1);
		AITEST_EQUAL("Test data should have been accessed 2 times", TestData.AccessCount, 2);

		// This test should get cached.
		const bool bResult2 = Query.IsTrue(ContextData);
		AITEST_TRUE("Query result should be true", bResult2);
		AITEST_EQUAL("Test data should have been accessed 2 times", TestData.AccessCount, 2);

		// Change and invalidate cached
		TestData.Value = 0;
		if (TestData.ValueChanged.IsBound())
		{
			TestData.ValueChanged.Broadcast();
		}
		
		// This test should get cached.
		const bool bResult3 = Query.IsTrue(ContextData);
		AITEST_FALSE("Query result should be false", bResult3);
		AITEST_EQUAL("Test data should have been accessed 4 times", TestData.AccessCount, 4);
		
		Query.Deactivate(ContextData);
		AITEST_FALSE("Query should not be active", Query.IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_CachedEval, "System.AI.WorldConditions.CachedEval");

struct FWorldConditionTest_EvalComplex : FAITestBase
{
	virtual bool InstantTest() override
	{
		FWorldConditionQuery Query;
		const bool bInitialized = Query.DebugInitialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(),
			{
				FWorldConditionEditable(/*Depth*/0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTest(0))),	//	if	(A
				FWorldConditionEditable(/*Depth*/1, EWorldConditionOperator::Or,   FConstStructView::Make(FWorldConditionTest(1))),	//	.	|| B)
				FWorldConditionEditable(/*Depth*/0, EWorldConditionOperator::And,  FConstStructView::Make(FWorldConditionTest(1))),	//	&&	(	(C
				FWorldConditionEditable(/*Depth*/2, EWorldConditionOperator::And,  FConstStructView::Make(FWorldConditionTest(1))),	//	.	.	&& D)
				FWorldConditionEditable(/*Depth*/1, EWorldConditionOperator::Or,   FConstStructView::Make(FWorldConditionTest(0))),	//	.	|| E)
			});

		AITEST_TRUE("Query should get initialized", bInitialized);
		
		FWorldConditionTestData TestData(1);

		const UWorldConditionTestSchema* TestSchema = GetDefault<UWorldConditionTestSchema>();
		FWorldConditionContextData ContextData(*TestSchema);
		const bool bContextDataSet = ContextData.SetContextData(TestSchema->GetValueRef(), &TestData);
		AITEST_TRUE("Query context data should be set", bContextDataSet);
		
		const bool bActivated = Query.Activate(GetWorld(), ContextData);
		AITEST_TRUE("Query should get activated", bActivated);
		AITEST_TRUE("Query should be active", Query.IsActive());
		
		const bool bResult1 = Query.IsTrue(ContextData);
		AITEST_TRUE("Query result1 should be true", bResult1);

		TestData.Value = 0;

		const bool bResult2 = Query.IsTrue(ContextData);
		AITEST_TRUE("Query result2 should be true", bResult2);

		Query.Deactivate(ContextData);
		AITEST_FALSE("Query should not be active", Query.IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_EvalComplex, "System.AI.WorldConditions.EvalComplex");

struct FWorldConditionTest_Empty : FAITestBase
{
	virtual bool InstantTest() override
	{
		FWorldConditionQuery Query;
		const bool bInitialized = Query.DebugInitialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(),{});

		AITEST_TRUE("Query should get initialized", bInitialized);
		
		FWorldConditionTestData TestData(1);
		const UWorldConditionTestSchema* TestSchema = GetDefault<UWorldConditionTestSchema>();
		FWorldConditionContextData ContextData(*TestSchema);
		const bool bContextDataSet = ContextData.SetContextData(TestSchema->GetValueRef(), &TestData);
		AITEST_TRUE("Query context data should be set", bContextDataSet);
		
		const bool bActivated = Query.Activate(GetWorld(), ContextData);
		AITEST_TRUE("Query should get activated", bActivated);
		AITEST_TRUE("Query should be active", Query.IsActive());
		
		const bool bResult1 = Query.IsTrue(ContextData);
		AITEST_TRUE("Query result1 should be true", bResult1);

		Query.Deactivate(ContextData);
		AITEST_FALSE("Query should not be active", Query.IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_Empty, "System.AI.WorldConditions.Empty");

struct FWorldConditionTest_FailingActivate : FAITestBase
{
	virtual bool InstantTest() override
	{
		FWorldConditionQuery Query;
		const bool bInitialized = Query.DebugInitialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(),
			{
				FWorldConditionEditable(0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTest(1))),
				FWorldConditionEditable(0, EWorldConditionOperator::And, FConstStructView::Make(FWorldConditionTest(1, false))) // this conditions fails to activate
			});

		AITEST_TRUE("Query should get initialized", bInitialized);
		
		const FWorldConditionTestData TestData(1);

		const UWorldConditionTestSchema* TestSchema = GetDefault<UWorldConditionTestSchema>();
		FWorldConditionContextData ContextData(*TestSchema);
		const bool bContextDataSet = ContextData.SetContextData(TestSchema->GetValueRef(), &TestData);
		AITEST_TRUE("Query context data should be set", bContextDataSet);

		{
			LOG_SCOPE_VERBOSITY_OVERRIDE(LogWorldCondition, ELogVerbosity::Fatal);
			const bool bActivated = Query.Activate(GetWorld(), ContextData);
			AITEST_FALSE("Query should not get activated", bActivated);
			AITEST_FALSE("Query should not be active", Query.IsActive());
		}

		const bool bResult = Query.IsTrue(ContextData);
		AITEST_FALSE("Query result should be false", bResult);

		Query.Deactivate(ContextData);
		AITEST_FALSE("Query should not be active", Query.IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_FailingActivate, "System.AI.WorldConditions.FailingActivate");

struct FWorldConditionTest_Serialization : FAITestBase
{
	virtual bool InstantTest() override
	{
		UWorldConditionOwnerClass* Owner = NewAutoDestroyObject<UWorldConditionOwnerClass>();
		const bool bInitialized = Owner->Definition.Initialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(),
			{
				FWorldConditionEditable(0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTest())),
				FWorldConditionEditable(0, EWorldConditionOperator::And, FConstStructView::Make(FWorldConditionTest())),
			});

		AITEST_TRUE("Query definition should get initialized", bInitialized);
		AITEST_EQUAL("Query successfully initialized definition should be valid", bInitialized, Owner->Definition.IsValid());

		TArray<uint8> Data;
		FMemoryWriter Writer(Data);
		FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/true);
		Owner->Serialize(WriterProxy);

		// Read back on new object
		Owner = NewAutoDestroyObject<UWorldConditionOwnerClass>();

		FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
		TGuardValue<UObject*> SerializedObjectScope(SerializeContext->SerializedObject, Owner);
		FMemoryReader Reader(Data);
		FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
		Owner->Serialize(ReaderProxy);

		AITEST_TRUE("Query definition should be initialized after loading", Owner->Definition.IsValid());
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_Serialization, "System.AI.WorldConditions.Serialization");

// Tests that FWorldConditionQuerySharedDefinition is properly linked when FWorldConditionQueryDefinition
// is nested inside a UStruct. In that case, UObject serialize context (SerializedObject) may be null
// when PostSerialize is called, and Link() must still be invoked.
struct FWorldConditionTest_SerializationInStruct : FAITestBase
{
	virtual bool InstantTest() override
	{
		UWorldConditionOwnerClassWithNestedStruct* Owner = NewAutoDestroyObject<UWorldConditionOwnerClassWithNestedStruct>();
		const bool bInitialized = Owner->Container.Definition.Initialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(),
			{
				FWorldConditionEditable(0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTest())),
				FWorldConditionEditable(0, EWorldConditionOperator::And, FConstStructView::Make(FWorldConditionTest())),
			});

		AITEST_TRUE("Query definition should get initialized", bInitialized);
		AITEST_TRUE("Query definition should be valid before serialization", Owner->Container.Definition.IsValid());

		TArray<uint8> Data;
		FMemoryWriter Writer(Data);
		FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/true);
		Owner->Serialize(WriterProxy);

		// Read back into a new object WITHOUT setting SerializedObject in the context.
		// This replicates the case where FWorldConditionQueryDefinition is nested inside a UStruct,
		// where the UObject serialize context is not set up by the caller (e.g., raw struct serialization).
		Owner = NewAutoDestroyObject<UWorldConditionOwnerClassWithNestedStruct>();
		FMemoryReader Reader(Data);
		FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
		Owner->Serialize(ReaderProxy);

		AITEST_TRUE("Query definition nested in a struct should be linked after loading", Owner->Container.Definition.IsValid());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_SerializationInStruct, "System.AI.WorldConditions.SerializationInStruct");

struct FWorldConditionTest_Description : FAITestBase
{
	virtual bool InstantTest() override
	{
		TArray<FWorldConditionEditable> Conditions = {
			FWorldConditionEditable(/*Depth*/0, EWorldConditionOperator::Copy, FConstStructView::Make(FWorldConditionTest(0))),	//	if	(A
			FWorldConditionEditable(/*Depth*/1, EWorldConditionOperator::Or,   FConstStructView::Make(FWorldConditionTest(1))),	//	.	|| B)
			FWorldConditionEditable(/*Depth*/0, EWorldConditionOperator::And,  FConstStructView::Make(FWorldConditionTest(1))),	//	&&	(	(C
			FWorldConditionEditable(/*Depth*/2, EWorldConditionOperator::And,  FConstStructView::Make(FWorldConditionTest(1))),	//	.	.	&& D)
			FWorldConditionEditable(/*Depth*/1, EWorldConditionOperator::Or,   FConstStructView::Make(FWorldConditionTest(0))),	//	.	|| E)
		};
		
		FWorldConditionQueryDefinition Definition;
		const bool bInitialized = Definition.Initialize(&GetWorld(), UWorldConditionTestSchema::StaticClass(), Conditions);
		AITEST_TRUE("Query definition should get initialized", bInitialized);

		auto GetDescriptionString = [](const FWorldConditionEditable& EditableCondition)
		{
			const FWorldConditionTest& Condition = EditableCondition.Condition.Get<const FWorldConditionTest>();
			return Condition.GetDescription().ToString();
		};
		
		const FString Expected =
			TEXT("IF ([") + GetDescriptionString(Conditions[0])
			+ TEXT("] OR [") + GetDescriptionString(Conditions[1])
			+ TEXT("]) AND (([") + GetDescriptionString(Conditions[2])
			+ TEXT("] AND [") + GetDescriptionString(Conditions[3])
			+ TEXT("]) OR [") + GetDescriptionString(Conditions[4])
			+ TEXT("])");

		const FText Desc = Definition.GetDescription();

		AITEST_EQUAL("Query text shold match expected result.", Desc.ToString(), Expected);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FWorldConditionTest_Description, "System.AI.WorldConditions.Description");

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
