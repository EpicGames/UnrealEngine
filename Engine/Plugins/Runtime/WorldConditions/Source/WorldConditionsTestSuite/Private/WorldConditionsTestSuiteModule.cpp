// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldConditionsTestSuiteModule.h"

#define LOCTEXT_NAMESPACE "WorldConditionsTestSuite"

class FWorldConditionsTestSuiteModule : public IWorldConditionsTestSuiteModule
{
};

IMPLEMENT_MODULE(FWorldConditionsTestSuiteModule, WorldConditionsTestSuite)

#undef LOCTEXT_NAMESPACE
