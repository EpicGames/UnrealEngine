// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldConditionContext.h"
#include "GameFramework/Actor.h"
#include "WorldConditionTestTypes.generated.h"

USTRUCT()
struct FWorldConditionTestData
{
	GENERATED_BODY()

	FWorldConditionTestData() = default;
	explicit FWorldConditionTestData(const int32 InValue) : Value(InValue) {} 

	FSimpleMulticastDelegate ValueChanged;

	int32 Value = 0;
	int32 AccessCount = 0;
};

UCLASS()
class UWorldConditionTestSchema : public UWorldConditionSchema
{
	GENERATED_BODY()
public:	
	explicit UWorldConditionTestSchema(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		ActorRef = AddContextDataDesc(TEXT("Actor"), AActor::StaticClass(), EWorldConditionContextDataType::Persistent);
		ValueRef = AddContextDataDesc(TEXT("Value"), FWorldConditionTestData::StaticStruct(), EWorldConditionContextDataType::Dynamic);
	}

	virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override
	{
		return InScriptStruct && InScriptStruct->IsChildOf(TBaseStructure<FWorldConditionBase>::Get());
	}

	FWorldConditionContextDataRef GetActorRef() const { return ActorRef; }
	FWorldConditionContextDataRef GetValueRef() const { return ValueRef; };

private:
	
	FWorldConditionContextDataRef ActorRef;
	FWorldConditionContextDataRef ValueRef;
};


UCLASS()
class UWorldConditionTestCachedSchema : public UWorldConditionSchema
{
	GENERATED_BODY()
public:	
	explicit UWorldConditionTestCachedSchema(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		ValueRef = AddContextDataDesc(TEXT("Value"), FWorldConditionTestData::StaticStruct(), EWorldConditionContextDataType::Persistent);
	}

	virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override
	{
		return InScriptStruct && InScriptStruct->IsChildOf(TBaseStructure<FWorldConditionBase>::Get());
	}

	FWorldConditionContextDataRef GetValueRef() const { return ValueRef; };

private:
	
	FWorldConditionContextDataRef ValueRef;
};

USTRUCT(meta=(Hidden))
struct FWorldConditionTest : public FWorldConditionBase
{
	GENERATED_BODY()

	FWorldConditionTest() = default;
	explicit FWorldConditionTest(const int32 InTestValue, const bool bInActivateResult = true) : TestValue(InTestValue), bActivateResult(bInActivateResult) {}
	
	virtual TObjectPtr<const UStruct>* GetRuntimeStateType() const override { return nullptr; }
	
	virtual bool Initialize(const UWorldConditionSchema& Schema) override
	{
		const UWorldConditionTestSchema* TestSchema = Cast<UWorldConditionTestSchema>(&Schema);
		if (TestSchema == nullptr)
		{
			return false;
		}
		ValueRef = TestSchema->GetValueRef();
		bCanCacheResult = false;
		return true;
	}
	
	virtual bool Activate(const FWorldConditionContext& Context) const override
	{
		return bActivateResult;
	}
	
	virtual FWorldConditionResult IsTrue(const FWorldConditionContext& Context) const override
	{
		FWorldConditionResult Result(EWorldConditionResultValue::IsFalse, bCanCacheResult);
		if (const FWorldConditionTestData* TestData = Context.GetContextDataPtr<FWorldConditionTestData>(ValueRef))
		{
			if (TestData->Value == TestValue)
			{
				Result.Value = EWorldConditionResultValue::IsTrue;
			}
		}
		return Result;
	}
	
	virtual void Deactivate(const FWorldConditionContext& Context) const override
	{
	}

	virtual FText GetDescription() const override
	{
		return FText::Format(FText::FromString(TEXT("Value == {0}")), FText::AsNumber(TestValue));
	}
	
protected:

	FWorldConditionContextDataRef ValueRef;
	
	int32 TestValue = 0;
	bool bActivateResult = true;
};


USTRUCT()
struct FWorldConditionTestState
{
	GENERATED_BODY()

	FDelegateHandle DelegateHandle;
};

USTRUCT(meta=(Hidden))
struct FWorldConditionTestCached : public FWorldConditionBase
{
	GENERATED_BODY()

	FWorldConditionTestCached() = default;
	explicit FWorldConditionTestCached(const int32 InTestValue) : TestValue(InTestValue) {}
	
	using FStateType = FWorldConditionTestState;
	
	virtual TObjectPtr<const UStruct>* GetRuntimeStateType() const override
	{
		static TObjectPtr<const UStruct> Ptr{FStateType::StaticStruct()};
		return &Ptr;
	}
	
	virtual bool Initialize(const UWorldConditionSchema& Schema) override
	{
		const UWorldConditionTestCachedSchema* TestSchema = Cast<UWorldConditionTestCachedSchema>(&Schema);
		if (TestSchema == nullptr)
		{
			return false;
		}
		ValueRef = TestSchema->GetValueRef();

		bCanCacheResult = Schema.GetContextDataTypeByRef(ValueRef) == EWorldConditionContextDataType::Persistent;
		
		return true;
	}
	
	virtual bool Activate(const FWorldConditionContext& Context) const override
	{
		if (Context.GetContextDataType(ValueRef) == EWorldConditionContextDataType::Persistent)
		{
			if (const FWorldConditionTestData* TestData = Context.GetContextDataPtr<FWorldConditionTestData>(ValueRef))
			{
				FStateType& State = Context.GetState(*this);
				FWorldConditionTestData* MutableTestData = const_cast<FWorldConditionTestData*>(TestData);
				State.DelegateHandle = MutableTestData->ValueChanged.AddLambda([InvalidationHandle = Context.GetInvalidationHandle(*this)]()
					{
						InvalidationHandle.InvalidateResult();
					});
				
				return true;
			}
			return false;
		}
		
		return true;
	}
	
	virtual FWorldConditionResult IsTrue(const FWorldConditionContext& Context) const override
	{
		FWorldConditionResult Result(EWorldConditionResultValue::IsFalse, bCanCacheResult);
		if (const FWorldConditionTestData* TestData = Context.GetContextDataPtr<FWorldConditionTestData>(ValueRef))
		{
			FWorldConditionTestData* MutableTestData = const_cast<FWorldConditionTestData*>(TestData);
			MutableTestData->AccessCount++;
			if (TestData->Value == TestValue)
			{
				Result.Value = EWorldConditionResultValue::IsTrue;
			}
		}
		return Result;
	}
	
	virtual void Deactivate(const FWorldConditionContext& Context) const override
	{
		FStateType& State = Context.GetState(*this);
		if (State.DelegateHandle.IsValid())
		{
			if (const FWorldConditionTestData* TestData = Context.GetContextDataPtr<FWorldConditionTestData>(ValueRef))
			{
				FWorldConditionTestData* MutableTestData = const_cast<FWorldConditionTestData*>(TestData);
				MutableTestData->ValueChanged.Remove(State.DelegateHandle);
				State.DelegateHandle.Reset();
			}
		}
	}

protected:

	FWorldConditionContextDataRef ValueRef;
	
	int32 TestValue = 0;
};

UCLASS(hidden)
class UWorldConditionOwnerClass : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Default")
	FWorldConditionQueryDefinition Definition;
};

USTRUCT()
struct FWorldConditionQueryDefinitionContainer
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Default")
	FWorldConditionQueryDefinition Definition;
};

UCLASS(hidden)
class UWorldConditionOwnerClassWithNestedStruct : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Default")
	FWorldConditionQueryDefinitionContainer Container;
};
