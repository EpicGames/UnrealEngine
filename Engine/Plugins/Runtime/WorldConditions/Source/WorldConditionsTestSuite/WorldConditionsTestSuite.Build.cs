// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WorldConditionsTestSuite : ModuleRules
	{
		public WorldConditionsTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new[] {
					"AITestSuite",
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
					"WorldConditions",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}