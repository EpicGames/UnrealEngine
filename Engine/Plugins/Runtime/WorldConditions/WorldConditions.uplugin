{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "World Conditions",
	"Description" : "General purpose cached conditions",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "http://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Modules":
	[
		{
			"Name": "WorldConditions",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name" : "WorldConditionsEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name": "WorldConditionsTestSuite",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins":
	[
		{
			"Name": "PropertyBindingUtils",
			"Enabled": true
		}
	]
}